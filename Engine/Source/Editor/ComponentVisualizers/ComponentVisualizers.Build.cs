// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class ComponentVisualizers : ModuleRules
{
	public ComponentVisualizers(ReadOnlyTargetRules Target) : base(Target)
	{
		// Enable truncation warnings in this module
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Warning;

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
                "InputCore",
				"Engine",
				"Slate",
                "SlateCore",
				"EditorFramework",
				"UnrealEd",
				"LevelEditor",
                "PropertyEditor",
				"AIModule",
				"ViewportInteraction"
			}
		);
	}
}
