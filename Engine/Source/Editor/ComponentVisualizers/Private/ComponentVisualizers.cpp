// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComponentVisualizers.h"

#include "AudioComponentVisualizer.h"
#include "ComponentVisualizer.h"
#include "Components/AudioComponent.h"
#include "Components/DecalComponent.h"
#include "Components/ForceFeedbackComponent.h"
#include "Components/PointLightComponent.h"
#include "Components/RectLightComponent.h"
#include "Components/SplineComponent.h"
#include "Components/SplineMeshComponent.h"
#include "Components/SpotLightComponent.h"
#include "Components/StereoLayerComponent.h"
#include "Components/WorldPartitionStreamingSourceComponent.h"
#include "Components/LocalFogVolumeComponent.h"
#include "ConstraintComponentVisualizer.h"
#include "DecalComponentVisualizer.h"
#include "Editor/UnrealEdEngine.h"
#include "ForceFeedbackComponentVisualizer.h"
#include "GameFramework/SpringArmComponent.h"
#include "HAL/PlatformCrt.h"
#include "Modules/ModuleManager.h"
#include "Perception/PawnSensingComponent.h"
#include "PhysicalAnimationComponentVisualizer.h"
#include "PhysicsEngine/PhysicalAnimationComponent.h"
#include "PhysicsEngine/PhysicsConstraintComponent.h"
#include "PhysicsEngine/PhysicsSpringComponent.h"
#include "PhysicsEngine/RadialForceComponent.h"
#include "PointLightComponentVisualizer.h"
#include "RadialForceComponentVisualizer.h"
#include "RectLightComponentVisualizer.h"
#include "SensingComponentVisualizer.h"
#include "SplineComponentVisualizer.h"
#include "SplineMeshComponentVisualizer.h"
#include "SpotLightComponentVisualizer.h"
#include "SpringArmComponentVisualizer.h"
#include "SpringComponentVisualizer.h"
#include "StereoLayerComponentVisualizer.h"
#include "UObject/Class.h"
#include "UnrealEdGlobals.h"
#include "WorldPartitionStreamingSourceComponentVisualizer.h"
#include "LocalFogVolumeComponentVisualizer.h"

IMPLEMENT_MODULE( FComponentVisualizersModule, ComponentVisualizers );

void FComponentVisualizersModule::StartupModule()
{
	RegisterComponentVisualizer(UPointLightComponent::StaticClass()->GetFName(), MakeShareable(new FPointLightComponentVisualizer));
	RegisterComponentVisualizer(USpotLightComponent::StaticClass()->GetFName(), MakeShareable(new FSpotLightComponentVisualizer));
	RegisterComponentVisualizer(URectLightComponent::StaticClass()->GetFName(), MakeShareable(new FRectLightComponentVisualizer));
	RegisterComponentVisualizer(UAudioComponent::StaticClass()->GetFName(), MakeShareable(new FAudioComponentVisualizer));
	RegisterComponentVisualizer(UForceFeedbackComponent::StaticClass()->GetFName(), MakeShareable(new FForceFeedbackComponentVisualizer));
	RegisterComponentVisualizer(URadialForceComponent::StaticClass()->GetFName(), MakeShareable(new FRadialForceComponentVisualizer));
	RegisterComponentVisualizer(UPhysicsConstraintComponent::StaticClass()->GetFName(), MakeShareable(new FConstraintComponentVisualizer));
	RegisterComponentVisualizer(UPhysicalAnimationComponent::StaticClass()->GetFName(), MakeShareable(new FPhysicsAnimationComponentVisualizer));
	RegisterComponentVisualizer(USpringArmComponent::StaticClass()->GetFName(), MakeShareable(new FSpringArmComponentVisualizer));
	RegisterComponentVisualizer(USplineComponent::StaticClass()->GetFName(), MakeShareable(new FSplineComponentVisualizer));
	RegisterComponentVisualizer(USplineMeshComponent::StaticClass()->GetFName(), MakeShareable(new FSplineMeshComponentVisualizer));
	RegisterComponentVisualizer(UPawnSensingComponent::StaticClass()->GetFName(), MakeShareable(new FSensingComponentVisualizer));
	RegisterComponentVisualizer(UPhysicsSpringComponent::StaticClass()->GetFName(), MakeShareable(new FSpringComponentVisualizer));
	RegisterComponentVisualizer(UDecalComponent::StaticClass()->GetFName(), MakeShareable(new FDecalComponentVisualizer));
	RegisterComponentVisualizer(UStereoLayerComponent::StaticClass()->GetFName(), MakeShareable(new FStereoLayerComponentVisualizer));
	RegisterComponentVisualizer(UWorldPartitionStreamingSourceComponent::StaticClass()->GetFName(), MakeShareable(new FWorldPartitionStreamingSourceComponentVisualizer));
	RegisterComponentVisualizer(ULocalFogVolumeComponent::StaticClass()->GetFName(), MakeShareable(new FLocalFogVolumeComponentVisualizer));
}

void FComponentVisualizersModule::ShutdownModule()
{
	if(GUnrealEd != NULL)
	{
		// Iterate over all class names we registered for
		for(FName ClassName : RegisteredComponentClassNames)
		{
			GUnrealEd->UnregisterComponentVisualizer(ClassName);
		}
	}
}

void FComponentVisualizersModule::RegisterComponentVisualizer(FName ComponentClassName, TSharedPtr<FComponentVisualizer> Visualizer)
{
	if (GUnrealEd != NULL)
	{
		GUnrealEd->RegisterComponentVisualizer(ComponentClassName, Visualizer);
	}

	RegisteredComponentClassNames.Add(ComponentClassName);

	if (Visualizer.IsValid())
	{
		Visualizer->OnRegister();
	}
}

void FComponentVisualizersModule::MuteComponentVisualizer(FName ComponentClassName)
{
	if (!GUnrealEd || !RegisteredComponentClassNames.Contains(ComponentClassName) || MutedComponentVisualizers.Contains(ComponentClassName))
	{
		return;
	}
	
	if (const TSharedPtr<FComponentVisualizer> Visualizer = GUnrealEd->FindComponentVisualizer(ComponentClassName))
	{
		// Unregister the visualizer.
		GUnrealEd->UnregisterComponentVisualizer(ComponentClassName);
		RegisteredComponentClassNames.Remove(ComponentClassName);

		// Cache the visualizer.
		MutedComponentVisualizers.Add(ComponentClassName, Visualizer);
	}
}

void FComponentVisualizersModule::UnmuteComponentVisualizer(FName ComponentClassName)
{
	if (!GUnrealEd || RegisteredComponentClassNames.Contains(ComponentClassName) || !MutedComponentVisualizers.Contains(ComponentClassName))
	{
		return;
	}

	if (const TSharedPtr<FComponentVisualizer>* VisualizerPtr = MutedComponentVisualizers.Find(ComponentClassName))
	{
		const TSharedPtr<FComponentVisualizer> Visualizer = *VisualizerPtr;

		// Register the visualizer, but don't call OnRegister (already called previously).
		GUnrealEd->RegisterComponentVisualizer(ComponentClassName, Visualizer);
		RegisteredComponentClassNames.Add(ComponentClassName);

		// Uncache the visualizer.
		MutedComponentVisualizers.Remove(ComponentClassName);
	}
}