// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConstraintComponentVisualizer.h"

#include "Chaos/ChaosEngineInterface.h"
#include "Components/ActorComponent.h"
#include "Engine/EngineTypes.h"
#include "HAL/PlatformCrt.h"
#include "Math/Box.h"
#include "Math/Color.h"
#include "Math/Quat.h"
#include "Math/Rotator.h"
#include "Math/Transform.h"
#include "Math/Vector.h"
#include "PhysicsEngine/ConstraintInstance.h"
#include "PhysicsEngine/PhysicsConstraintComponent.h"
#include "PrimitiveDrawingUtils.h"
#include "Templates/Casts.h"

static const FColor	JointFrame1Color(255,0,0);
static const FColor JointFrame2Color(0,0,255);

void FConstraintComponentVisualizer::DrawVisualization( const UActorComponent* Component, const FSceneView* View, FPrimitiveDrawInterface* PDI )
{
	const UPhysicsConstraintComponent* ConstraintComp = Cast<const UPhysicsConstraintComponent>(Component);
	if(ConstraintComp != NULL)
	{
		const FConstraintInstance& Instance = ConstraintComp->ConstraintInstance;

		FTransform Con1Frame, Con2Frame;

		// If constraint is created, use the calculated frame
		if(Instance.IsValidConstraintInstance())
		{
			FTransform BodyTransform1 = ConstraintComp->GetBodyTransform(EConstraintFrame::Frame1);
			FTransform BodyTransform2 = ConstraintComp->GetBodyTransform(EConstraintFrame::Frame2);
			BodyTransform1.RemoveScaling();
			BodyTransform2.RemoveScaling();

			Con1Frame = Instance.GetRefFrame(EConstraintFrame::Frame1);
			Con2Frame = Instance.GetRefFrame(EConstraintFrame::Frame2);

			const float LastKnownScale = Instance.GetLastKnownScale();

			if(ConstraintComp->GetPhysicsObject(EConstraintFrame::Frame1))
			{
				Con1Frame.ScaleTranslation(LastKnownScale);
			}
			

			Con2Frame.SetScale3D(Con2Frame.GetScale3D() * LastKnownScale);

			if (ConstraintComp->GetPhysicsObject(EConstraintFrame::Frame2))
			{
				Con2Frame.ScaleTranslation(LastKnownScale);
			}
			
			Con1Frame *= BodyTransform1;
			Con2Frame *= BodyTransform2;
		}
		// Otherwise use the component frame
		else
		{
			Con1Frame = ConstraintComp->GetComponentTransform();
			Con1Frame.RemoveScaling();
			Con2Frame = ConstraintComp->GetComponentTransform();
			Con2Frame.SetRotation(Con2Frame.GetRotation() * ConstraintComp->ConstraintInstance.AngularRotationOffset.Quaternion());
			Con1Frame.RemoveScaling();
		}

		FBox Body1Box = ConstraintComp->GetBodyBox(EConstraintFrame::Frame1);
		FBox Body2Box = ConstraintComp->GetBodyBox(EConstraintFrame::Frame2);

		// Draw constraint information
		Instance.DrawConstraint(PDI, 1.f, 1.f, true, true, Con1Frame, Con2Frame, false, false);

		// Draw boxes to indicate bodies connected by joint.
		if(Body1Box.IsValid)
		{
			PDI->DrawLine( Con1Frame.GetTranslation(), Body1Box.GetCenter(), JointFrame1Color, SDPG_World );
			DrawWireBox(PDI, Body1Box, JointFrame1Color, SDPG_World );
		}

		if(Body2Box.IsValid)
		{
			PDI->DrawLine( Con2Frame.GetTranslation(), Body2Box.GetCenter(), JointFrame2Color, SDPG_World );
			DrawWireBox(PDI, Body2Box, JointFrame2Color, SDPG_World );
		}
	}
}
