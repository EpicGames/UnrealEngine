// Copyright Epic Games, Inc. All Rights Reserved.

#include "DecalComponentVisualizer.h"

#include "Components/ActorComponent.h"
#include "Components/DecalComponent.h"
#include "Engine/EngineTypes.h"
#include "Math/Axis.h"
#include "Math/Color.h"
#include "Math/Matrix.h"
#include "Math/Transform.h"
#include "PrimitiveDrawingUtils.h"
#include "Templates/Casts.h"



void FDecalComponentVisualizer::DrawVisualization( const UActorComponent* Component, const FSceneView* View, FPrimitiveDrawInterface* PDI )
{
	const UDecalComponent* DecalComponent = Cast<const UDecalComponent>(Component);
	if(DecalComponent)
	{
		const FMatrix LocalToWorld = DecalComponent->GetComponentTransform().ToMatrixWithScale();
		
		const FLinearColor DrawColor = FColor(0, 157, 0, 255);

		DrawOrientedWireBox(PDI, LocalToWorld.GetOrigin(), LocalToWorld.GetScaledAxis( EAxis::X ), LocalToWorld.GetScaledAxis( EAxis::Y ), LocalToWorld.GetScaledAxis( EAxis::Z ), DecalComponent->DecalSize, DrawColor, SDPG_World);
	}
}
