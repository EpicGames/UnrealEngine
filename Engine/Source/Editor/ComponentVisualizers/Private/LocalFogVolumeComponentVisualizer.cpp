// Copyright Epic Games, Inc. All Rights Reserved.

#include "LocalFogVolumeComponentVisualizer.h"

#include "Components/ActorComponent.h"
#include "Components/LocalFogVolumeComponent.h"
#include "SceneView.h"
#include "PrimitiveDrawingUtils.h"

void FLocalFogVolumeComponentVisualizer::DrawVisualization( const UActorComponent* Component, const FSceneView* View, FPrimitiveDrawInterface* PDI )
{
	if(View->Family->EngineShowFlags.CompositeEditorPrimitives)
	{
		const ULocalFogVolumeComponent* LocalFogVolume = Cast<const ULocalFogVolumeComponent>(Component);
		if(LocalFogVolume != NULL)
		{
			FTransform LocalFogVolumeTransform = LocalFogVolume->GetComponentTransform();

			const float MaximumAxisScale = LocalFogVolumeTransform.GetMaximumAxisScale() * ULocalFogVolumeComponent::GetBaseVolumeSize();
			LocalFogVolumeTransform.SetScale3D(FVector(MaximumAxisScale, MaximumAxisScale, MaximumAxisScale));

			// Draw local fog volume spherical shape radius
			DrawWireSphereAutoSides(PDI, LocalFogVolumeTransform, FColor(200, 255, 255), 1.0f, SDPG_World);
		}
	}
}


