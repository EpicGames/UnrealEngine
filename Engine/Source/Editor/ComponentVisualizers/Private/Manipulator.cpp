// Copyright Epic Games, Inc. All Rights Reserved.
#include "Manipulator.h"

#include "Components/SceneComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/EngineTypes.h"
#include "Engine/StaticMesh.h"
#include "HAL/Platform.h"
#include "Materials/Material.h"
#include "Math/Transform.h"
#include "Misc/AssertionMacros.h"
#include "UObject/ConstructorHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Manipulator)

PRAGMA_DISABLE_DEPRECATION_WARNINGS


class UActorComponent;
class UViewportInteractor;
struct FHitResult;

AManipulator::AManipulator()
	: Super()
{
	static ConstructorHelpers::FObjectFinder<UStaticMesh> MeshRef(TEXT("/Engine/VREditor/TransformGizmo/SM_Sequencer_Node"));
	UStaticMesh* StaticMesh = MeshRef.Object;
	check(StaticMesh != nullptr);

	static ConstructorHelpers::FObjectFinder<UMaterial> MaterialRef(TEXT("/Engine/VREditor/TransformGizmo/Main"));
	UMaterial* Material = MaterialRef.Object;
	check(Material != nullptr);

	const bool bTransient = true;
	USceneComponent* SceneComponent = CreateDefaultSubobject<USceneComponent>(TEXT("SceneComponent"), bTransient);
	check(SceneComponent != nullptr);
	this->RootComponent = SceneComponent;

	StaticMeshComponent = CreateDefaultSubobject<UStaticMeshComponent>(TEXT("KeyMesh"));
	StaticMeshComponent->SetCanEverAffectNavigation(false);
	StaticMeshComponent->SetMobility(EComponentMobility::Movable);
	StaticMeshComponent->SetupAttachment(RootComponent);
	StaticMeshComponent->SetStaticMesh(StaticMesh);
	StaticMeshComponent->CreateAndSetMaterialInstanceDynamicFromMaterial(0, Material);

	StaticMeshComponent->SetCollisionEnabled(ECollisionEnabled::QueryOnly);
	StaticMeshComponent->SetCollisionResponseToAllChannels(ECR_Ignore);
	StaticMeshComponent->SetCollisionResponseToChannel(ECC_Visibility, ECR_Block);

	StaticMeshComponent->SetGenerateOverlapEvents(false);
	StaticMeshComponent->bCastDynamicShadow = false;
	StaticMeshComponent->bCastStaticShadow = false;
	StaticMeshComponent->bAffectDistanceFieldLighting = false;
	StaticMeshComponent->bAffectDynamicIndirectLighting = false;

	AssociatedComponent = nullptr;
}

void AManipulator::PostEditMove(bool bFinished)
{
	Super::PostEditMove(bFinished);

	if (AssociatedComponent != nullptr)
	{
		const FTransform& Transform = GetActorTransform();
		AssociatedComponent->SetWorldTransform(Transform);
	}
}

bool AManipulator::IsEditorOnly() const
{
	return true;
}

void AManipulator::OnPressed(UViewportInteractor* Interactor, const FHitResult& InHitResult, bool& bOutResultedInDrag)
{

}

void AManipulator::OnHover(UViewportInteractor* Interactor)
{

}

void AManipulator::OnHoverEnter(UViewportInteractor* Interactor, const FHitResult& InHitResult)
{

}

void AManipulator::OnHoverLeave(UViewportInteractor* Interactor, const UActorComponent* NewComponent)
{

}

void AManipulator::OnDragRelease(UViewportInteractor* Interactor)
{

}

void AManipulator::SetAssociatedComponent(USceneComponent* SceneComponent)
{
	AssociatedComponent = SceneComponent;

	// Set the initial transform for the manipulator to be the same as the associated component.
	if (AssociatedComponent != nullptr)
	{
		SetActorTransform(AssociatedComponent->GetComponentToWorld());	
	}
}


PRAGMA_ENABLE_DEPRECATION_WARNINGS
