// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicalAnimationComponentVisualizer.h"

#include "Components/ActorComponent.h"
#include "PhysicsEngine/PhysicalAnimationComponent.h"
#include "Templates/Casts.h"

void FPhysicsAnimationComponentVisualizer::DrawVisualization( const UActorComponent* Component, const FSceneView* View, FPrimitiveDrawInterface* PDI )
{
	if(const UPhysicalAnimationComponent* PhysAnimComp = Cast<const UPhysicalAnimationComponent>(Component))
	{
		PhysAnimComp->DebugDraw(PDI);
	}
}
