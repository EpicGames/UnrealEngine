// Copyright Epic Games, Inc. All Rights Reserved.

#include "PointLightComponentVisualizer.h"

#include "Components/ActorComponent.h"
#include "Components/PointLightComponent.h"
#include "Containers/ContainersFwd.h"
#include "Engine/EngineTypes.h"
#include "Engine/Texture.h"
#include "Engine/TextureDefines.h"
#include "Engine/TextureLightProfile.h"
#include "Math/Axis.h"
#include "Math/Color.h"
#include "Math/Float16.h"
#include "Math/Rotator.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "PrimitiveDrawingUtils.h"
#include "SceneView.h"
#include "ShowFlags.h"
#include "Templates/Casts.h"
#include "Templates/Function.h"
#include "UObject/ObjectPtr.h"



void FPointLightComponentVisualizer::DrawVisualization( const UActorComponent* Component, const FSceneView* View, FPrimitiveDrawInterface* PDI )
{
	if(View->Family->EngineShowFlags.LightRadius)
	{
		const UPointLightComponent* PointLightComp = Cast<const UPointLightComponent>(Component);
		if(PointLightComp != NULL)
		{
			FTransform LightTM = PointLightComp->GetComponentTransform();
			LightTM.RemoveScaling();

			// Draw light radius
			DrawWireSphereAutoSides(PDI, LightTM, FColor(200, 255, 255), PointLightComp->AttenuationRadius, SDPG_World);

			// Draw point light source shape
			DrawWireCapsule(PDI, LightTM.GetTranslation(), LightTM.GetUnitAxis( EAxis::X ), LightTM.GetUnitAxis( EAxis::Y ), LightTM.GetUnitAxis( EAxis::Z ),
							FColor(231, 239, 0, 255), PointLightComp->SourceRadius, 0.5f * PointLightComp->SourceLength + PointLightComp->SourceRadius, 25, SDPG_World);

			if (PointLightComp->IESTexture)
			{
				LightProfileVisualizer.DrawVisualization( PointLightComp->IESTexture, LightTM, View, PDI );
			}
		}
	}
}

namespace TextureLightProfileVisualizerImpl
{
	struct FTextureLightProfileData
	{
		const TArray64< uint8 >& Data;

		int32 SizeX;
		int32 SizeY;
		int32 BytesPerPixel;
	};

	float FilterLightProfile(const FTextureLightProfileData& TextureLightProfileData, const float X, const float Y)
	{
		const int32 SizeX = TextureLightProfileData.SizeX;
		const int32 SizeY = TextureLightProfileData.SizeY;
		const int32 BytesPerPixel = TextureLightProfileData.BytesPerPixel;

		// not 100% like GPU hardware but simple and almost the same
		float UnNormalizedX = FMath::Clamp(X * static_cast<float>(SizeX), 0.0f, static_cast<float>(SizeX - 1));
		float UnNormalizedY = FMath::Clamp(Y * static_cast<float>(SizeY), 0.0f,static_cast<float>(SizeY - 1));

		int32 X0 = (uint32)UnNormalizedX;
		int32 X1 = FMath::Min(X0 + 1, SizeX - 1);

		int32 Y0 = (uint32)UnNormalizedY;
		int32 Y1 = FMath::Min(Y0 + 1, SizeY - 1);

		float XFraction = UnNormalizedX - static_cast<float>(X0);
		float YFraction = UnNormalizedY - static_cast<float>(Y0);

		float V00 = reinterpret_cast<const FFloat16*>(&TextureLightProfileData.Data[ (Y0 * SizeX * BytesPerPixel) + (X0 * BytesPerPixel) ])->GetFloat();
		float V10 = reinterpret_cast<const FFloat16*>(&TextureLightProfileData.Data[ (Y1 * SizeX * BytesPerPixel) + (X0 * BytesPerPixel) ])->GetFloat();
		float V01 = reinterpret_cast<const FFloat16*>(&TextureLightProfileData.Data[ (Y0 * SizeX * BytesPerPixel) + (X1 * BytesPerPixel) ])->GetFloat();
		float V11 = reinterpret_cast<const FFloat16*>(&TextureLightProfileData.Data[ (Y1 * SizeX * BytesPerPixel) + (X1 * BytesPerPixel) ])->GetFloat();

		float V0 = FMath::Lerp(V00, V10, YFraction);
		float V1 = FMath::Lerp(V01, V11, YFraction);

		return FMath::Lerp(V0, V1, XFraction);
	}

	float ComputeLightProfileMultiplier(const FTextureLightProfileData& TextureLightProfileData, FVector WorldPosition, FVector LightPosition, FVector LightDirection, FTransform InvLightTransform)
	{
		FVector ToLight = (LightPosition - WorldPosition).GetSafeNormal();
		FVector LocalToLight = InvLightTransform.TransformVector( ToLight );

		// -1..1
		double DotProd = FVector::DotProduct(ToLight, LightDirection);
		// -PI..PI (this distortion could be put into the texture but not without quality loss or more memory)
		float Angle = (float)FMath::Asin(DotProd);
		// 0..1
		float NormAngle = Angle / UE_PI + 0.5f;

		float TangentAngle = (float)FMath::Atan2( -LocalToLight.Z, -LocalToLight.Y ); // -Y represents 0/360 horizontal angle and we're rotating counter-clockwise
		float NormTangentAngle = (TangentAngle / (UE_PI * 2.f) + 0.5f);

		return FilterLightProfile( TextureLightProfileData, NormAngle, NormTangentAngle );
	}

	struct FPolarCoordinates
	{
		static constexpr int32 MaxAltitude = 360;
		static constexpr int32 MaxAzimuth = 180;

		FPolarCoordinates() = default;

		FPolarCoordinates( int32 InVAngle, int32 InHAngle )
			: Altitude( InVAngle )
			, Azimuth( InHAngle )
		{
			Normalize();
		}

		/** Clamps altitude between [0, MaxAltitude[ and altitude between [0, MaxAzimuth[ */
		void Normalize()
		{
			int32 NormalizedAzimuth = Azimuth;
			int32 NormalizedAltitude = Altitude;

			if ( Azimuth >= MaxAzimuth )
			{
				NormalizedAzimuth -= MaxAzimuth;
				NormalizedAltitude = MaxAzimuth - NormalizedAltitude;
			}
			else if ( Azimuth < 0 )
			{
				NormalizedAzimuth += MaxAzimuth;
				NormalizedAltitude = MaxAzimuth - NormalizedAltitude;
			}

			if ( NormalizedAltitude >= MaxAltitude )
			{
				NormalizedAltitude -= MaxAltitude;
			}
			else if ( NormalizedAltitude < 0 )
			{
				NormalizedAltitude += MaxAltitude;
			}

			Azimuth = NormalizedAzimuth;
			Altitude = NormalizedAltitude;
		}

		int32 Altitude = 0;
		int32 Azimuth = 0;
	};

	struct FPolarSampler
	{
		static constexpr int32 Step = 5; // Sample at each 5 degrees

		static void ForEach( TFunction< void( FPolarCoordinates PolarCoordinates ) > Func )
		{
			FPolarCoordinates PolarCoordinates;

			for ( PolarCoordinates.Altitude = 0; PolarCoordinates.Altitude < FPolarCoordinates::MaxAltitude; PolarCoordinates.Altitude += Step )
			{
				for ( PolarCoordinates.Azimuth = 0; PolarCoordinates.Azimuth < FPolarCoordinates::MaxAzimuth; PolarCoordinates.Azimuth += Step )
				{
					Func( PolarCoordinates );
				}
			}
		}

		static int32 ComputePolarIndex( const FPolarCoordinates& PolarCoordinates )
		{
			int32 VAngleIndex = PolarCoordinates.Altitude / Step;
			int32 HAngleIndex = PolarCoordinates.Azimuth / Step;

			int32 MaxHAngleIndex = FPolarCoordinates::MaxAzimuth / Step;

			return ( HAngleIndex + ( VAngleIndex * MaxHAngleIndex ) );
		}

		static FVector ComputePosition( const FTransform& Referential, const FPolarCoordinates& PolarCoordinates, float Distance )
		{
			FVector Direction = FVector::ForwardVector;
			FRotator Rotator( PolarCoordinates.Altitude, PolarCoordinates.Azimuth, 0.f );

			FVector RotatedDirection = Rotator.RotateVector( Direction ) * Distance;
			RotatedDirection = Referential.TransformVector( RotatedDirection );

			const FVector Origin = Referential.GetTranslation();
			const FVector Position = Origin + RotatedDirection;

			return Position;
		}

		static FPolarCoordinates GetVerticalNeighbor( const FPolarCoordinates& PolarCoordinates )
		{
			return FPolarCoordinates( PolarCoordinates.Altitude + Step, PolarCoordinates.Azimuth );
		}

		static FPolarCoordinates GetHorizontalNeighbor( const FPolarCoordinates& PolarCoordinates )
		{
			return FPolarCoordinates( PolarCoordinates.Altitude, PolarCoordinates.Azimuth + Step );
		}
	};
}

FTextureLightProfileVisualizer::FTextureLightProfileVisualizer()
	: CachedLightProfile( nullptr )
{
}

void FTextureLightProfileVisualizer::DrawVisualization(UTextureLightProfile* TextureLightProfile, const FTransform& LightTM, const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	using namespace TextureLightProfileVisualizerImpl;

	if (!UpdateIntensitiesCache(TextureLightProfile, LightTM))
	{
		return;
	}
	
	FPolarSampler::ForEach(
		[ &LightTM, PDI, this ]( FPolarCoordinates PolarCoordinates ) -> void
		{
			float PolarDistance = IntensitiesCache[ FPolarSampler::ComputePolarIndex( PolarCoordinates ) ] * 100.f;
			const FVector Position = FPolarSampler::ComputePosition( LightTM, PolarCoordinates, PolarDistance );

			// Draw line to vertical neighbor
			{
				const FPolarCoordinates VerticalNeighbor = FPolarSampler::GetVerticalNeighbor( PolarCoordinates );
				int32 NextVAngleIntensityIndex = FPolarSampler::ComputePolarIndex( VerticalNeighbor );
				float NextVAngleIntensity = IntensitiesCache[ NextVAngleIntensityIndex ];

				if ( !FMath::IsNearlyZero( NextVAngleIntensity ) )
				{
					float VerticalNeighborPolarDistance = IntensitiesCache[ FPolarSampler::ComputePolarIndex( VerticalNeighbor ) ] * 100.f;

					FVector VerticalNeighborPosition = FPolarSampler::ComputePosition( LightTM, VerticalNeighbor, VerticalNeighborPolarDistance );
					PDI->DrawLine( Position, VerticalNeighborPosition, FColor(231, 239, 0, 255), SDPG_World );
				}
			}

			// Draw line to horizontal neighbor
			{
				const FPolarCoordinates HorizontalNeighbor = FPolarSampler::GetHorizontalNeighbor( PolarCoordinates );
				int32 NextHAngleIntensityIndex = FPolarSampler::ComputePolarIndex( HorizontalNeighbor );
				float NextHAngleIntensity = IntensitiesCache[ NextHAngleIntensityIndex ];

				if ( !FMath::IsNearlyZero( NextHAngleIntensity ) )
				{
					float HorizontalNeighborPolarDistance = IntensitiesCache[ FPolarSampler::ComputePolarIndex( HorizontalNeighbor ) ] * 100.f;

					FVector HorizontalNeighborPosition = FPolarSampler::ComputePosition( LightTM, HorizontalNeighbor, HorizontalNeighborPolarDistance );
					PDI->DrawLine( Position, HorizontalNeighborPosition, FColor(231, 239, 0, 255), SDPG_World );
				}
			}
		}
	);
}

bool FTextureLightProfileVisualizer::UpdateIntensitiesCache(UTextureLightProfile* TextureLightProfile, const FTransform& LightTM)
{
	using namespace TextureLightProfileVisualizerImpl;

	// Only RGBA16F is supported for IES light profiles
	if ( !TextureLightProfile || ! TextureLightProfile->Source.IsValid() || TextureLightProfile->Source.GetFormat() != TSF_RGBA16F )
	{
		CachedLightProfile = nullptr;
		IntensitiesCache.Empty();
		return false; 
	}

	if ( CachedLightProfile == TextureLightProfile )
	{
		return true;
	}

	CachedLightProfile = TextureLightProfile;

	constexpr int32 Step = 5; // Sample at each 5 degrees
	
	const FVector LightDirection = LightTM.GetUnitAxis( EAxis::X );
	const FTransform InvLightTransform = LightTM.Inverse();

	const FVector StartPos = LightTM.GetTranslation();

	TArray64< uint8 > MipData;
	verify( TextureLightProfile->Source.GetMipData( MipData, 0 ) );

	FTextureLightProfileData TextureLightProfileData{ MipData, (int32)TextureLightProfile->Source.GetSizeX(), (int32)TextureLightProfile->Source.GetSizeY(), (int32)TextureLightProfile->Source.GetBytesPerPixel() };

	int32 NumIntensities = ( FPolarCoordinates::MaxAltitude / FPolarSampler::Step ) * ( FPolarCoordinates::MaxAzimuth / FPolarSampler::Step );

	IntensitiesCache.Empty( NumIntensities );
	IntensitiesCache.AddZeroed( NumIntensities );

	FPolarSampler::ForEach(
		[ &LightTM, &TextureLightProfileData, &StartPos, &LightDirection, &InvLightTransform, this ]( FPolarCoordinates PolarCoordinates ) -> void
		{
			const FVector EndPos = FPolarSampler::ComputePosition( LightTM, PolarCoordinates, 1.f );

			float LightProfileIntensity = ComputeLightProfileMultiplier( TextureLightProfileData, EndPos, StartPos, LightDirection, InvLightTransform );

			int32 SampleIndex = FPolarSampler::ComputePolarIndex( PolarCoordinates );

			IntensitiesCache[ SampleIndex ] = LightProfileIntensity;
		}
	);
	return true;
}

