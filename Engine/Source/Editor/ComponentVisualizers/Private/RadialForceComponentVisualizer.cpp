// Copyright Epic Games, Inc. All Rights Reserved.

#include "RadialForceComponentVisualizer.h"

#include "Components/ActorComponent.h"
#include "Engine/EngineTypes.h"
#include "HAL/PlatformCrt.h"
#include "Math/Color.h"
#include "Math/Transform.h"
#include "PhysicsEngine/RadialForceComponent.h"
#include "PrimitiveDrawingUtils.h"
#include "Templates/Casts.h"



void FRadialForceComponentVisualizer::DrawVisualization( const UActorComponent* Component, const FSceneView* View, FPrimitiveDrawInterface* PDI )
{
	const URadialForceComponent* ForceComp = Cast<const URadialForceComponent>(Component);
	if(ForceComp != NULL)
	{
		FTransform TM = ForceComp->GetComponentTransform();
		TM.RemoveScaling();

		// Draw light radius
		DrawWireSphereAutoSides(PDI, TM, FColor(200, 255, 255), ForceComp->Radius, SDPG_World);
	}
}
