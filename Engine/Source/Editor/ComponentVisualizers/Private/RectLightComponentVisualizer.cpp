// Copyright Epic Games, Inc. All Rights Reserved.

#include "RectLightComponentVisualizer.h"

#include "Components/ActorComponent.h"
#include "Components/RectLightComponent.h"
#include "Engine/EngineTypes.h"
#include "Math/Box.h"
#include "Math/Color.h"
#include "Math/Transform.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "PrimitiveDrawingUtils.h"
#include "SceneView.h"
#include "ShowFlags.h"
#include "Templates/Casts.h"

int32 GVisualizeCullingBarnDoors = 0;
static FAutoConsoleVariableRef CVarVisualizeCullingBarnDoors(
	TEXT("r.RectLight.VisualizeCullingBarnDoors"),
	GVisualizeCullingBarnDoors,
	TEXT("Whether to render a visualization of the barn doors used to cull the rect light."),
	ECVF_RenderThreadSafe
);

void FRectLightComponentVisualizer::DrawVisualization( const UActorComponent* Component, const FSceneView* View, FPrimitiveDrawInterface* PDI )
{
	if(View->Family->EngineShowFlags.LightRadius)
	{
		const URectLightComponent* RectLightComp = Cast<const URectLightComponent>(Component);
		if(RectLightComp != NULL)
		{
			FTransform LightTM = RectLightComp->GetComponentTransform();
			LightTM.RemoveScaling();

			// Draw light radius
			DrawWireSphereAutoSides(PDI, LightTM, FColor(200, 255, 255), RectLightComp->AttenuationRadius, SDPG_World);

			FBox Box(
				FVector( 0.0f, -0.5f * RectLightComp->SourceWidth, -0.5f * RectLightComp->SourceHeight ),
				FVector( 0.0f,  0.5f * RectLightComp->SourceWidth,  0.5f * RectLightComp->SourceHeight )
			);

			const FColor ElementColor(231, 239, 0, 255);

			auto DrawBarnRect = [&](const FVector& P0, const FVector& P1, const FVector& P2, const FVector& P3, const FColor& Color)
			{
				FVector TP0 = LightTM.TransformPosition(P0);
				FVector TP1 = LightTM.TransformPosition(P1);
				FVector TP2 = LightTM.TransformPosition(P2);
				FVector TP3 = LightTM.TransformPosition(P3);
				PDI->DrawLine(TP0, TP1, Color, SDPG_World);
				PDI->DrawLine(TP1, TP2, Color, SDPG_World);
				PDI->DrawLine(TP2, TP3, Color, SDPG_World);
				PDI->DrawLine(TP3, TP0, Color, SDPG_World);
			};

			const float BarnMaxAngle = GetRectLightBarnDoorMaxAngle();
			const float AngleRad	= FMath::DegreesToRadians(FMath::Clamp(RectLightComp->BarnDoorAngle, 0.f, BarnMaxAngle));
			const float BarnDepth	= FMath::Cos(AngleRad) * RectLightComp->BarnDoorLength;
			const float BarnExtent	= FMath::Sin(AngleRad) * RectLightComp->BarnDoorLength;

			FVector Corners[8];

			// +SourceWidth
			{
				FVector P0(0.0f,		+0.5f * RectLightComp->SourceWidth,					-0.5f * RectLightComp->SourceHeight);
				FVector P1(0.0f,		+0.5f * RectLightComp->SourceWidth,					+0.5f * RectLightComp->SourceHeight);
				FVector P2(BarnDepth, 	+0.5f * RectLightComp->SourceWidth + BarnExtent,	+0.5f * RectLightComp->SourceHeight + BarnExtent);
				FVector P3(BarnDepth, 	+0.5f * RectLightComp->SourceWidth + BarnExtent,	-0.5f * RectLightComp->SourceHeight - BarnExtent);
				Corners[0] =  P3;
				Corners[1] =  P2;

				DrawBarnRect(P0, P1, P2, P3, ElementColor);
			}
			// +SourceHeight
			{
				FVector P0(0.0f,		-0.5f * RectLightComp->SourceWidth,	+0.5f * RectLightComp->SourceHeight);
				FVector P1(0.0f,		+0.5f * RectLightComp->SourceWidth,	+0.5f * RectLightComp->SourceHeight);
				FVector P2(BarnDepth,	+0.5f * RectLightComp->SourceWidth + BarnExtent,	+0.5f * RectLightComp->SourceHeight + BarnExtent);
				FVector P3(BarnDepth,	-0.5f * RectLightComp->SourceWidth - BarnExtent,	+0.5f * RectLightComp->SourceHeight + BarnExtent);
				Corners[2] =  P2;
				Corners[3] =  P3;

				DrawBarnRect(P0, P1, P2, P3, ElementColor);
			}
			// -SourceWidth
			{
				FVector P0(0.0f,		-0.5f * RectLightComp->SourceWidth,					-0.5f * RectLightComp->SourceHeight);
				FVector P1(0.0f,		-0.5f * RectLightComp->SourceWidth,					+0.5f * RectLightComp->SourceHeight);
				FVector P2(BarnDepth,	-0.5f * RectLightComp->SourceWidth - BarnExtent,	+0.5f * RectLightComp->SourceHeight + BarnExtent);
				FVector P3(BarnDepth,	-0.5f * RectLightComp->SourceWidth - BarnExtent,	-0.5f * RectLightComp->SourceHeight - BarnExtent);
				Corners[4] =  P2;
				Corners[5] =  P3;

				DrawBarnRect(P0, P1, P2, P3, ElementColor);
			}
			// -SourceHeight
			{
				FVector P0(0.0f,		-0.5f * RectLightComp->SourceWidth,	-0.5f * RectLightComp->SourceHeight);
				FVector P1(0.0f,		+0.5f * RectLightComp->SourceWidth,	-0.5f * RectLightComp->SourceHeight);
				FVector P2(BarnDepth,	+0.5f * RectLightComp->SourceWidth + BarnExtent,	-0.5f * RectLightComp->SourceHeight - BarnExtent);
				FVector P3(BarnDepth,	-0.5f * RectLightComp->SourceWidth - BarnExtent,	-0.5f * RectLightComp->SourceHeight - BarnExtent);
				Corners[6] =  P3;
				Corners[7] =  P2;

				DrawBarnRect(P0, P1, P2, P3, ElementColor);
			
			}

			DrawWireBox(PDI, LightTM.ToMatrixNoScale(), Box, ElementColor, SDPG_World);

			if(GVisualizeCullingBarnDoors != 0)
			{
				const FColor CullingRectColor(255, 0, 0, 255);

				{
					float HorizontalBarnExtent;
					float HorizontalBarnDepth;
					CalculateRectLightCullingBarnExtentAndDepth(RectLightComp->SourceWidth, RectLightComp->BarnDoorLength, AngleRad, RectLightComp->AttenuationRadius, HorizontalBarnExtent, HorizontalBarnDepth);

					TStaticArray<FVector, 8> HorizontalCorners;
					CalculateRectLightBarnCorners(RectLightComp->SourceWidth, RectLightComp->SourceHeight, HorizontalBarnExtent, HorizontalBarnDepth, HorizontalCorners);

					DrawBarnRect(HorizontalCorners[0], HorizontalCorners[2], HorizontalCorners[3], HorizontalCorners[1], CullingRectColor);
					DrawBarnRect(HorizontalCorners[4], HorizontalCorners[6], HorizontalCorners[7], HorizontalCorners[5], CullingRectColor);
				}

				{
					float VerticalBarnExtent;
					float VerticalBarnDepth;
					CalculateRectLightCullingBarnExtentAndDepth(RectLightComp->SourceHeight, RectLightComp->BarnDoorLength, AngleRad, RectLightComp->AttenuationRadius, VerticalBarnExtent, VerticalBarnDepth);

					TStaticArray<FVector, 8> VerticalCorners;
					CalculateRectLightBarnCorners(RectLightComp->SourceWidth, RectLightComp->SourceHeight, VerticalBarnExtent, VerticalBarnDepth, VerticalCorners);

					DrawBarnRect(VerticalCorners[0], VerticalCorners[4], VerticalCorners[6], VerticalCorners[2], CullingRectColor);
					DrawBarnRect(VerticalCorners[5], VerticalCorners[7], VerticalCorners[3], VerticalCorners[1], CullingRectColor);
				}

			}
		}
	}
}
