// Copyright Epic Games, Inc. All Rights Reserved.

#include "SensingComponentVisualizer.h"

#include "Components/ActorComponent.h"
#include "Containers/Array.h"
#include "Engine/EngineTypes.h"
#include "Math/Color.h"
#include "Math/Transform.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "Perception/PawnSensingComponent.h"
#include "PrimitiveDrawingUtils.h"
#include "SceneView.h"
#include "ShowFlags.h"
#include "Templates/Casts.h"


void FSensingComponentVisualizer::DrawVisualization(const UActorComponent* Component, const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	if (View->Family->EngineShowFlags.VisualizeSenses)
	{
		const UPawnSensingComponent* Senses = Cast<const UPawnSensingComponent>(Component);
		if (Senses != NULL)
		{
			const FTransform Transform = FTransform(Senses->GetSensorRotation(), Senses->GetSensorLocation());

			//LOS hearing
			if (Senses->LOSHearingThreshold > 0.0f)
			{
				DrawWireSphere(PDI, Transform, FColor::Yellow, Senses->LOSHearingThreshold, 16, SDPG_World);
			}

			//Hearing
			if (Senses->HearingThreshold > 0.0f)
			{
				DrawWireSphere(PDI, Transform, FColor::Cyan, Senses->HearingThreshold, 16, SDPG_World);
			}

			// Sight
			if (Senses->SightRadius > 0.0f)
			{
				TArray<FVector> Verts;
				DrawWireCone(PDI, Verts, Transform, Senses->SightRadius, Senses->GetPeripheralVisionAngle(), 10, FColor::Green, SDPG_World);
			}
		}
	}
}
