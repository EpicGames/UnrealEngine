// Copyright Epic Games, Inc. All Rights Reserved.

#include "SplineComponentVisualizer.h"
#include "CoreMinimal.h"
#include "Algo/AnyOf.h"
#include "Components/BrushComponent.h"
#include "CollisionQueryParams.h"
#include "Engine/HitResult.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/Commands.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GameFramework/Volume.h"
#include "Styling/AppStyle.h"
#include "UnrealWidgetFwd.h"
#include "Editor.h"
#include "Selection.h"
#include "EditorViewportClient.h"
#include "EditorViewportCommands.h"
#include "LevelEditorActions.h"
#include "Components/SplineComponent.h"
#include "SceneView.h"
#include "ScopedTransaction.h"
#include "ActorEditorUtils.h"
#include "UObject/UObjectIterator.h"
#include "WorldCollision.h"
#include "Widgets/Docking/SDockTab.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "SplineGeneratorPanel.h"
#include "EngineUtils.h"
#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "LevelEditorViewport.h"
#include "UnrealEdGlobals.h"
#include "Editor/UnrealEdEngine.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/AxisDisplayInfo.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Views/SListView.h"
#include "Features/IModularFeatures.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SplineComponentVisualizer)

IMPLEMENT_HIT_PROXY(HSplineVisProxy, HComponentVisProxy);
IMPLEMENT_HIT_PROXY(HSplineKeyProxy, HSplineVisProxy);
IMPLEMENT_HIT_PROXY(HSplineAttributeKeyProxy, HSplineVisProxy);
IMPLEMENT_HIT_PROXY(HSplineSegmentProxy, HSplineVisProxy);
IMPLEMENT_HIT_PROXY(HSplineTangentHandleProxy, HSplineVisProxy);

#define LOCTEXT_NAMESPACE "SplineComponentVisualizer"
DEFINE_LOG_CATEGORY_STATIC(LogSplineComponentVisualizer, Log, All)

#define VISUALIZE_SPLINE_UPVECTORS 0

namespace SplineComponentVisualizerLocals
{
	bool GRebuildPDICacheEveryFrame = false;
	FAutoConsoleVariableRef CVarUseSplineCurves(
		TEXT("SplineComponentVisualizer.RebuildPDICacheEveryFrame"),
		GRebuildPDICacheEveryFrame,
		TEXT("When true, the spline is fully resampled for rendering every frame. Otherwise, the cache is invalidated only when necessary.")
	);

	bool GIgnoreSpuriousDeselect = true;
	FAutoConsoleVariableRef CVarIgnoreSpuriousDeselect(
		TEXT("SplineComponentVisualizer.IgnoreSpuriousDeselect"),
		GIgnoreSpuriousDeselect,
		TEXT("When true, OnDeselectedInEditor ignores deselect broadcasts whose component we are no longer editing or that is still in the editor's selection set. Set false to restore the prior unconditional EndEditing.")
	);
	
	bool IsEnabledForSpline(const USplineComponent* InSplineComponent)
	{
		auto ModularFeatureName = ISplineComponentVisualizerSuppressor::GetModularFeatureName();
	
		if (IModularFeatures::Get().IsModularFeatureAvailable(ModularFeatureName))
		{
			return !IModularFeatures::Get().GetModularFeature<ISplineComponentVisualizerSuppressor>(ModularFeatureName).ShouldSuppress(InSplineComponent);
		}
		
		return true;
	}
	
	// This is mostly modeled on FindNearestVisibleObjectHit_Internal in ModelingSceneSnappingManager.cpp,
	// which we probably can't access because it lives in a plugin.
	// TODO: Perhaps this code should live in some common utility place? Certainly if we do this again.
	bool RaycastWorld(const UWorld* World, FEditorViewportClient* ViewportClient, FViewport* Viewport, FHitResult& HitResultOut)
	{
		FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(
			ViewportClient->Viewport,
			ViewportClient->GetScene(),
			ViewportClient->EngineShowFlags));
		// this View is deleted by the FSceneViewFamilyContext destructor
		FSceneView* View = ViewportClient->CalcSceneView(&ViewFamily);

		FViewportCursorLocation MouseViewportRay(View, ViewportClient, Viewport->GetMouseX(), Viewport->GetMouseY());
		FRay Ray(MouseViewportRay.GetOrigin(), MouseViewportRay.GetDirection());

		FCollisionObjectQueryParams ObjectQueryParams(FCollisionObjectQueryParams::AllObjects);
		FCollisionQueryParams QueryParams = FCollisionQueryParams::DefaultQueryParam;
		QueryParams.bTraceComplex = true;

		TArray<FHitResult> OutHits;
		FVector RayEnd = (FVector)(Ray.PointAt(HALF_WORLD_MAX));
		if (World->LineTraceMultiByObjectType(OutHits, (FVector)Ray.Origin, RayEnd, ObjectQueryParams, QueryParams) == false)
		{
			return false;
		}

		double NearestVisible = TNumericLimits<double>::Max();
		for (const FHitResult& CurResult : OutHits)
		{
			UPrimitiveComponent* Component = CurResult.Component.Get();
			AActor* Actor = CurResult.GetActor();

			// Don't use volumes
			if (Cast<UBrushComponent>(Component) != nullptr &&
				Cast<AVolume>(Actor) != nullptr)
			{
				continue;
			}

			// Ignore invisible things
			if ((Actor && Actor->IsHidden())
				|| (Component && !Component->IsVisibleInEditor()))
			{
				continue;
			}

			if (CurResult.Distance < NearestVisible)
			{
				HitResultOut = CurResult;
				NearestVisible = CurResult.Distance;
			}
		}

		return NearestVisible < TNumericLimits<double>::Max();
	}

	bool IsCurvePointType(EInterpCurveMode SplinePointType)
	{
		FInterpCurvePoint<float> Dummy;
		Dummy.InterpMode = SplinePointType;
		return Dummy.IsCurveKey();
	}

	bool IsCurvePointType(ESplinePointType::Type SplinePointType)
	{
		return IsCurvePointType(ConvertSplinePointTypeToInterpCurveMode(SplinePointType));
	}
}

int32 USplineComponentVisualizerSelectionState::GetVerifiedLastKeyIndexSelected(const int32 InNumSplinePoints) const
{
	check(LastKeyIndexSelected != INDEX_NONE);
	check(LastKeyIndexSelected >= 0);
	check(LastKeyIndexSelected < InNumSplinePoints);
	return LastKeyIndexSelected;
}

void USplineComponentVisualizerSelectionState::GetVerifiedSelectedTangentHandle(const int32 InNumSplinePoints, int32& OutSelectedTangentHandle, ESelectedTangentHandle& OutSelectedTangentHandleType) const
{
	check(SelectedTangentHandle != INDEX_NONE);
	check(SelectedTangentHandle >= 0);
	check(SelectedTangentHandle < InNumSplinePoints);
	check(SelectedTangentHandleType != ESelectedTangentHandle::None);
	OutSelectedTangentHandle = SelectedTangentHandle;
	OutSelectedTangentHandleType = SelectedTangentHandleType;
}
void USplineComponentVisualizerSelectionState::Reset()
{
	SplinePropertyPath = FComponentPropertyPath();
	ClearSelectedKeys();
	CachedRotation = FQuat();
	ClearSelectedSegmentIndex();
	ClearSelectedTangentHandle();
	ClearSelectedAttribute();
}

void USplineComponentVisualizerSelectionState::ClearSelectedKeys()
{
	SelectedKeys.Reset();
	LastKeyIndexSelected = INDEX_NONE;
}

void USplineComponentVisualizerSelectionState::ClearSelectedSegmentIndex()
{
	SelectedSegmentIndex = INDEX_NONE;
}

void USplineComponentVisualizerSelectionState::ClearSelectedTangentHandle()
{
	SelectedTangentHandle = INDEX_NONE;
	SelectedTangentHandleType = ESelectedTangentHandle::None;
}

bool USplineComponentVisualizerSelectionState::IsSplinePointSelected(const int32 InIndex) const
{
	return SelectedKeys.Contains(InIndex);
}

/** Define commands for the spline component visualizer */
class FSplineComponentVisualizerCommands : public TCommands<FSplineComponentVisualizerCommands>
{
public:
	FSplineComponentVisualizerCommands() : TCommands <FSplineComponentVisualizerCommands>
	(
		"SplineComponentVisualizer",	// Context name for fast lookup
		LOCTEXT("SplineComponentVisualizer", "Spline Component Visualizer"),	// Localized context name for displaying
		NAME_None,	// Parent
		FAppStyle::GetAppStyleSetName()
	)
	{
	}

	virtual void RegisterCommands() override
	{
		UI_COMMAND(DeleteKey, "Delete Spline Point", "Delete the currently selected spline point.", EUserInterfaceActionType::Button, FInputChord(EKeys::Delete));
		UI_COMMAND(DuplicateKey, "Duplicate Spline Point", "Duplicate the currently selected spline point.", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(AddKey, "Add Spline Point Here", "Add a new spline point at the cursor location.", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(SelectAll, "Select All Spline Points", "Select all spline points.", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(SelectNextSplinePoint, "Select Next Spline Point", "Select next spline point.", EUserInterfaceActionType::Button, FInputChord(EKeys::Period));
		UI_COMMAND(SelectPrevSplinePoint, "Select Prev Spline Point", "Select prev spline point.", EUserInterfaceActionType::Button, FInputChord(EKeys::Comma));
		UI_COMMAND(AddNextSplinePoint, "Add Next Spline Point", "Add next spline point.", EUserInterfaceActionType::Button, FInputChord(EKeys::Period, EModifierKey::Shift));
		UI_COMMAND(AddPrevSplinePoint, "Add Prev Spline Point", "Add prev spline point.", EUserInterfaceActionType::Button, FInputChord(EKeys::Comma, EModifierKey::Shift));
		UI_COMMAND(ResetToUnclampedTangent, "Unclamped Tangent", "Reset the tangent for this spline point to its default unclamped value.", EUserInterfaceActionType::Button, FInputChord(EKeys::T));
		UI_COMMAND(ResetToClampedTangent, "Clamped Tangent", "Reset the tangent for this spline point to its default clamped value.", EUserInterfaceActionType::Button, FInputChord(EKeys::T, EModifierKey::Shift));
		UI_COMMAND(SetKeyToCurve, "Curve", "Set spline point to Curve type", EUserInterfaceActionType::RadioButton, FInputChord());
		UI_COMMAND(SetKeyToLinear, "Linear", "Set spline point to Linear type", EUserInterfaceActionType::RadioButton, FInputChord());
		UI_COMMAND(SetKeyToConstant, "Constant", "Set spline point to Constant type", EUserInterfaceActionType::RadioButton, FInputChord());
		UI_COMMAND(FocusViewportToSelection, "Focus Selected", "Moves the camera in front of the selection", EUserInterfaceActionType::Button, FInputChord(EKeys::F));
		UI_COMMAND(SnapKeyToNearestSplinePoint, "Snap to Nearest Spline Point", "Snap selected spline point to nearest non-adjacent spline point on current or nearby spline.", EUserInterfaceActionType::Button, FInputChord(EKeys::P, EModifierKey::Shift));
		UI_COMMAND(AlignKeyToNearestSplinePoint, "Align to Nearest Spline Point", "Align selected spline point to nearest non-adjacent spline point on current or nearby spline.", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(AlignKeyPerpendicularToNearestSplinePoint, "Align Perpendicular to Nearest Spline Point", "Align perpendicular selected spline point to nearest non-adjacent spline point on current or nearby spline.", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(SnapKeyToActor, "Snap to Actor", "Snap selected spline point to actor, Ctrl-LMB to select the actor after choosing this option.", EUserInterfaceActionType::Button, FInputChord(EKeys::P, (EModifierKey::Alt | EModifierKey::Shift)));
		UI_COMMAND(AlignKeyToActor, "Align to Actor", "Align selected spline point to actor, Ctrl-LMB to select the actor after choosing this option.", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(AlignKeyPerpendicularToActor, "Align Perpendicular to Actor", "Align perpendicular  selected spline point to actor, Ctrl-LMB to select the actor after choosing this option.", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(ToggleSnapTangentAdjustments, "Allow Tangents Updates On Snap", "Allow tangents to update when performing snap operations on points.", EUserInterfaceActionType::ToggleButton, FInputChord());

#define AXIS_UI_COMMAND(CommandName, Axis, LabelText, TooltipText, ButtonType) \
	FUICommandInfo::MakeCommandInfo( \
		AsShared(), \
		CommandName, \
		TEXT(#CommandName), \
		FText::Format(LOCTEXT(#CommandName, LabelText), AxisDisplayInfo::GetAxisDisplayName(Axis)), \
		FText::Format(LOCTEXT(#CommandName TEXT("_ToolTip"), TooltipText), AxisDisplayInfo::GetAxisDisplayName(Axis)), \
		FSlateIcon(), \
		ButtonType, \
		FInputChord() \
		); \

		AXIS_UI_COMMAND(SnapAllToSelectedY, EAxisList::Left, "Snap All To Selected {0}", "Snap all spline points to selected spline point world {0} position.", EUserInterfaceActionType::Button)
		AXIS_UI_COMMAND(SnapAllToSelectedZ, EAxisList::Up, "Snap All To Selected {0}", "Snap all spline points to selected spline point world {0} position.", EUserInterfaceActionType::Button)
		AXIS_UI_COMMAND(SnapAllToSelectedX, EAxisList::Forward ,"Snap All To Selected {0}", "Snap all spline points to selected spline point world {0} position.", EUserInterfaceActionType::Button)

		AXIS_UI_COMMAND(SnapToLastSelectedY, EAxisList::Left, "Snap To Last Selected {0}", "Snap selected spline points to world {0} position of last selected spline point.", EUserInterfaceActionType::Button)
		AXIS_UI_COMMAND(SnapToLastSelectedZ, EAxisList::Up, "Snap To Last Selected {0}", "Snap selected spline points to world {0} position of last selected spline point.", EUserInterfaceActionType::Button)
		AXIS_UI_COMMAND(SnapToLastSelectedX, EAxisList::Forward, "Snap To Last Selected {0}", "Snap selected spline points to world {0} position of last selected spline point.", EUserInterfaceActionType::Button)

		AXIS_UI_COMMAND(SetLockedAxisY, EAxisList::Left, "{0}", "Fix {0} axis when adding new spline points.", EUserInterfaceActionType::RadioButton)
		AXIS_UI_COMMAND(SetLockedAxisZ, EAxisList::Up, "{0}", "Fix {0} axis when adding new spline points.", EUserInterfaceActionType::RadioButton)
		AXIS_UI_COMMAND(SetLockedAxisX, EAxisList::Forward, "{0}", "Fix {0} axis when adding new spline points.", EUserInterfaceActionType::RadioButton)
#undef AXIS_UI_COMMAND


		UI_COMMAND(StraightenToNext, "Straighten To Next Point", "Straighten selected points toward next sequential point", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(StraightenToPrevious, "Straighten To Previous Point", "Straighten selected points toward previous sequential point", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(SetLockedAxisNone, "None", "New spline point axis is not fixed.", EUserInterfaceActionType::RadioButton, FInputChord());
		UI_COMMAND(VisualizeRollAndScale, "Visualize Roll and Scale", "Whether the visualization should show roll and scale on this spline.", EUserInterfaceActionType::ToggleButton, FInputChord());
		UI_COMMAND(DiscontinuousSpline, "Allow Discontinuous Splines", "Whether the visualization allows Arrive and Leave tangents to be set separately.", EUserInterfaceActionType::ToggleButton, FInputChord());
		UI_COMMAND(ToggleClosedLoop, "Closed Loop", "Toggle the Closed Loop setting of the spline", EUserInterfaceActionType::ToggleButton, FInputChord());
		UI_COMMAND(ResetToDefault, "Reset to Default", "Reset this spline to its archetype default.", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(AddAttributeKey, "Add Attribute Here", "Add a new attribute value at the cursor location.", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(DeleteAttributeKey, "Delete Attribute", "Delete the currently selected attribute.", EUserInterfaceActionType::Button, FInputChord());
	}

public:
	/** Delete key */
	TSharedPtr<FUICommandInfo> DeleteKey;

	/** Duplicate key */
	TSharedPtr<FUICommandInfo> DuplicateKey;

	/** Add key */
	TSharedPtr<FUICommandInfo> AddKey;

	/** Select all */
	TSharedPtr<FUICommandInfo> SelectAll;

	/** Select next spline point */
	TSharedPtr<FUICommandInfo> SelectNextSplinePoint;

	/** Select prev spline point */
	TSharedPtr<FUICommandInfo> SelectPrevSplinePoint;

	/** Add next spline point */
	TSharedPtr<FUICommandInfo> AddNextSplinePoint;

	/** Add prev spline point */
	TSharedPtr<FUICommandInfo> AddPrevSplinePoint;

	/** Reset to unclamped tangent */
	TSharedPtr<FUICommandInfo> ResetToUnclampedTangent;

	/** Reset to clamped tangent */
	TSharedPtr<FUICommandInfo> ResetToClampedTangent;

	/** Set spline key to Curve type */
	TSharedPtr<FUICommandInfo> SetKeyToCurve;

	/** Set spline key to Linear type */
	TSharedPtr<FUICommandInfo> SetKeyToLinear;

	/** Set spline key to Constant type */
	TSharedPtr<FUICommandInfo> SetKeyToConstant;

	/** Focus on selection */
	TSharedPtr<FUICommandInfo> FocusViewportToSelection;

	/** Snap key to nearest spline point on another spline component */
	TSharedPtr<FUICommandInfo> SnapKeyToNearestSplinePoint;

	/** Align key to nearest spline point on another spline component */
	TSharedPtr<FUICommandInfo> AlignKeyToNearestSplinePoint;

	/** Align key perpendicular to nearest spline point on another spline component */
	TSharedPtr<FUICommandInfo> AlignKeyPerpendicularToNearestSplinePoint;

	/** Snap key to nearest actor */
	TSharedPtr<FUICommandInfo> SnapKeyToActor;

	/** Align key to nearest actor */
	TSharedPtr<FUICommandInfo> AlignKeyToActor;

	/** Align key perpendicular to nearest actor */
	TSharedPtr<FUICommandInfo> AlignKeyPerpendicularToActor;

	/** Turn On / Off Tangent updates when snapping points*/
	TSharedPtr<FUICommandInfo> ToggleSnapTangentAdjustments;

	/** Snap all spline points to selected point world X position*/
	TSharedPtr<FUICommandInfo> SnapAllToSelectedX;

	/** Snap all spline points to selected point world Y position */
	TSharedPtr<FUICommandInfo> SnapAllToSelectedY;

	/** Snap all spline points to selected point world Z position */
	TSharedPtr<FUICommandInfo> SnapAllToSelectedZ;

	/** Snap selected spline points to last selected point world X position */
	TSharedPtr<FUICommandInfo> SnapToLastSelectedX;

	/** Snap selected spline points to last selected point world Y position */
	TSharedPtr<FUICommandInfo> SnapToLastSelectedY;

	/** Snap selected spline points to last selected point world Z position */
	TSharedPtr<FUICommandInfo> SnapToLastSelectedZ;

	/** Straighten tangents to align directly toward Next spline points */
	TSharedPtr<FUICommandInfo> StraightenToNext;

	/** Straighten tangents to align directly toward Previous spline points */
	TSharedPtr<FUICommandInfo> StraightenToPrevious;

	/** No axis is locked when adding new spline points */
	TSharedPtr<FUICommandInfo> SetLockedAxisNone;

	/** Lock X axis when adding new spline points */
	TSharedPtr<FUICommandInfo> SetLockedAxisX;

	/** Lock Y axis when adding new spline points */
	TSharedPtr<FUICommandInfo> SetLockedAxisY;

	/** Lock Z axis when adding new spline points */
	TSharedPtr<FUICommandInfo> SetLockedAxisZ;

	/** Whether the visualization should show roll and scale */
	TSharedPtr<FUICommandInfo> VisualizeRollAndScale;

	/** Whether we allow separate Arrive / Leave tangents, resulting in a discontinuous spline */
	TSharedPtr<FUICommandInfo> DiscontinuousSpline;

	/** Toggle the Closed Loop setting of the spline */
	TSharedPtr<FUICommandInfo> ToggleClosedLoop;

	/** Reset this spline to its default */
	TSharedPtr<FUICommandInfo> ResetToDefault;

	/** Add attribute key */
	TSharedPtr<FUICommandInfo> AddAttributeKey;

	/** Delete attribute key */
	TSharedPtr<FUICommandInfo> DeleteAttributeKey;
};

TWeakPtr<SWindow> FSplineComponentVisualizer::WeakExistingWindow;

FSplineComponentVisualizer::FSplineComponentVisualizer()
	: FComponentVisualizer()
	, bAllowDuplication(true)
	, bDuplicatingSplineKey(false)
	, bUpdatingAddSegment(false)
	, DuplicateDelay(0)
	, DuplicateDelayAccumulatedDrag(FVector::ZeroVector)
	, DuplicateCacheSplitSegmentParam(0.0f)
	, AddKeyLockedAxis(EAxis::None)
	, bIsSnappingToActor(false)
	, SnapToActorMode(ESplineComponentSnapMode::Snap)
{
	FSplineComponentVisualizerCommands::Register();

	SplineComponentVisualizerActions = MakeShareable(new FUICommandList);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	SplineCurvesProperty = nullptr;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	for (const FName& Property : USplineComponent::GetSplinePropertyNames())
	{
		SplineProperties.Add(FindFProperty<FProperty>(USplineComponent::StaticClass(), Property));
	}
	
	SelectionState = NewObject<USplineComponentVisualizerSelectionState>(GetTransientPackage(), TEXT("SelectionState"), RF_Transactional);

	IModularFeatures::Get().RegisterModularFeature(ISplineDetailsProvider::GetModularFeatureName(), this);

	OnObjectTransactedHandle = FCoreUObjectDelegates::OnObjectTransacted.AddLambda([this](UObject* Object, const class FTransactionObjectEvent& Transaction)
	{
		if (Object == SelectionState && Transaction.GetEventType() == ETransactionObjectEventType::UndoRedo)
		{
			// Because the set of elements to draw is a function of a spline and a selection,
			// changing the selection invalidates all caches, not just the cached elements for any particular spline.
			PDICache.Empty();
		}
	});
}

void FSplineComponentVisualizer::OnRegister()
{
	const auto& Commands = FSplineComponentVisualizerCommands::Get();

	SplineComponentVisualizerActions->MapAction(
		Commands.DeleteKey,
		FExecuteAction::CreateSP(this, &FSplineComponentVisualizer::OnDeleteKey),
		FCanExecuteAction::CreateSP(this, &FSplineComponentVisualizer::CanDeleteKey));

	SplineComponentVisualizerActions->MapAction(
		Commands.DuplicateKey,
		FExecuteAction::CreateSP(this, &FSplineComponentVisualizer::OnDuplicateKey),
		FCanExecuteAction::CreateSP(this, &FSplineComponentVisualizer::IsKeySelectionValid));

	SplineComponentVisualizerActions->MapAction(
		Commands.AddKey,
		FExecuteAction::CreateSP(this, &FSplineComponentVisualizer::OnAddKeyToSegment),
		FCanExecuteAction::CreateSP(this, &FSplineComponentVisualizer::CanAddKeyToSegment));

	SplineComponentVisualizerActions->MapAction(
		Commands.SelectAll,
		FExecuteAction::CreateSP(this, &FSplineComponentVisualizer::OnSelectAllSplinePoints),
		FCanExecuteAction::CreateSP(this, &FSplineComponentVisualizer::CanSelectSplinePoints));

	SplineComponentVisualizerActions->MapAction(
		Commands.SelectNextSplinePoint,
		FExecuteAction::CreateSP(this, &FSplineComponentVisualizer::OnSelectPrevNextSplinePoint, true, false),
		FCanExecuteAction::CreateSP(this, &FSplineComponentVisualizer::CanSelectSplinePoints));

	SplineComponentVisualizerActions->MapAction(
		Commands.SelectPrevSplinePoint,
		FExecuteAction::CreateSP(this, &FSplineComponentVisualizer::OnSelectPrevNextSplinePoint, false, false),
		FCanExecuteAction::CreateSP(this, &FSplineComponentVisualizer::CanSelectSplinePoints));

	SplineComponentVisualizerActions->MapAction(
		Commands.AddNextSplinePoint,
		FExecuteAction::CreateSP(this, &FSplineComponentVisualizer::OnSelectPrevNextSplinePoint, true, true),
		FCanExecuteAction::CreateSP(this, &FSplineComponentVisualizer::CanSelectSplinePoints));

	SplineComponentVisualizerActions->MapAction(
		Commands.AddPrevSplinePoint,
		FExecuteAction::CreateSP(this, &FSplineComponentVisualizer::OnSelectPrevNextSplinePoint, false, true),
		FCanExecuteAction::CreateSP(this, &FSplineComponentVisualizer::CanSelectSplinePoints));

	SplineComponentVisualizerActions->MapAction(
		Commands.ResetToUnclampedTangent,
		FExecuteAction::CreateSP(this, &FSplineComponentVisualizer::OnResetToAutomaticTangent, CIM_CurveAuto),
		FCanExecuteAction::CreateSP(this, &FSplineComponentVisualizer::CanResetToAutomaticTangent, CIM_CurveAuto));

	SplineComponentVisualizerActions->MapAction(
		Commands.ResetToClampedTangent,
		FExecuteAction::CreateSP(this, &FSplineComponentVisualizer::OnResetToAutomaticTangent, CIM_CurveAutoClamped),
		FCanExecuteAction::CreateSP(this, &FSplineComponentVisualizer::CanResetToAutomaticTangent, CIM_CurveAutoClamped));

	SplineComponentVisualizerActions->MapAction(
		Commands.SetKeyToCurve,
		FExecuteAction::CreateSP(this, &FSplineComponentVisualizer::OnSetKeyType, CIM_CurveAuto),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FSplineComponentVisualizer::IsKeyTypeSet, CIM_CurveAuto));

	SplineComponentVisualizerActions->MapAction(
		Commands.SetKeyToLinear,
		FExecuteAction::CreateSP(this, &FSplineComponentVisualizer::OnSetKeyType, CIM_Linear),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FSplineComponentVisualizer::IsKeyTypeSet, CIM_Linear));

	SplineComponentVisualizerActions->MapAction(
		Commands.SetKeyToConstant,
		FExecuteAction::CreateSP(this, &FSplineComponentVisualizer::OnSetKeyType, CIM_Constant),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FSplineComponentVisualizer::IsKeyTypeSet, CIM_Constant));

	SplineComponentVisualizerActions->MapAction(
		Commands.FocusViewportToSelection,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("CAMERA ALIGN ACTIVEVIEWPORTONLY") ) )
	);

	SplineComponentVisualizerActions->MapAction(
		Commands.SnapKeyToNearestSplinePoint,
		FExecuteAction::CreateSP(this, &FSplineComponentVisualizer::OnSnapKeyToNearestSplinePoint, ESplineComponentSnapMode::Snap),
		FCanExecuteAction::CreateSP(this, &FSplineComponentVisualizer::IsSingleKeySelected));

	SplineComponentVisualizerActions->MapAction(
		Commands.AlignKeyToNearestSplinePoint,
		FExecuteAction::CreateSP(this, &FSplineComponentVisualizer::OnSnapKeyToNearestSplinePoint, ESplineComponentSnapMode::AlignToTangent),
		FCanExecuteAction::CreateSP(this, &FSplineComponentVisualizer::IsSingleKeySelected));

	SplineComponentVisualizerActions->MapAction(
		Commands.AlignKeyPerpendicularToNearestSplinePoint,
		FExecuteAction::CreateSP(this, &FSplineComponentVisualizer::OnSnapKeyToNearestSplinePoint, ESplineComponentSnapMode::AlignPerpendicularToTangent),
		FCanExecuteAction::CreateSP(this, &FSplineComponentVisualizer::IsSingleKeySelected));

	SplineComponentVisualizerActions->MapAction(
		Commands.SnapKeyToActor,
		FExecuteAction::CreateSP(this, &FSplineComponentVisualizer::OnSnapKeyToActor, ESplineComponentSnapMode::Snap),
		FCanExecuteAction::CreateSP(this, &FSplineComponentVisualizer::IsSingleKeySelected));

	SplineComponentVisualizerActions->MapAction(
		Commands.AlignKeyToActor,
		FExecuteAction::CreateSP(this, &FSplineComponentVisualizer::OnSnapKeyToActor, ESplineComponentSnapMode::AlignToTangent),
		FCanExecuteAction::CreateSP(this, &FSplineComponentVisualizer::IsSingleKeySelected));

	SplineComponentVisualizerActions->MapAction(
		Commands.AlignKeyPerpendicularToActor,
		FExecuteAction::CreateSP(this, &FSplineComponentVisualizer::OnSnapKeyToActor, ESplineComponentSnapMode::AlignPerpendicularToTangent),
		FCanExecuteAction::CreateSP(this, &FSplineComponentVisualizer::IsSingleKeySelected));
	
	SplineComponentVisualizerActions->MapAction(
		Commands.ToggleSnapTangentAdjustments,
		FExecuteAction::CreateSP(this, &FSplineComponentVisualizer::OnToggleSnapTangentAdjustment),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FSplineComponentVisualizer::IsSnapTangentAdjustment));

	SplineComponentVisualizerActions->MapAction(
		Commands.SnapAllToSelectedX,
		FExecuteAction::CreateSP(this, &FSplineComponentVisualizer::OnSnapAllToAxis, EAxis::X),
		FCanExecuteAction::CreateSP(this, &FSplineComponentVisualizer::IsSingleKeySelected));

	SplineComponentVisualizerActions->MapAction(
		Commands.SnapAllToSelectedY,
		FExecuteAction::CreateSP(this, &FSplineComponentVisualizer::OnSnapAllToAxis, EAxis::Y),
		FCanExecuteAction::CreateSP(this, &FSplineComponentVisualizer::IsSingleKeySelected));

	SplineComponentVisualizerActions->MapAction(
		Commands.SnapAllToSelectedZ,
		FExecuteAction::CreateSP(this, &FSplineComponentVisualizer::OnSnapAllToAxis, EAxis::Z),
		FCanExecuteAction::CreateSP(this, &FSplineComponentVisualizer::IsSingleKeySelected));

	SplineComponentVisualizerActions->MapAction(
		Commands.SnapToLastSelectedX,
		FExecuteAction::CreateSP(this, &FSplineComponentVisualizer::OnSnapSelectedToAxis, EAxis::X),
		FCanExecuteAction::CreateSP(this, &FSplineComponentVisualizer::AreMultipleKeysSelected));

	SplineComponentVisualizerActions->MapAction(
		Commands.SnapToLastSelectedY,
		FExecuteAction::CreateSP(this, &FSplineComponentVisualizer::OnSnapSelectedToAxis, EAxis::Y),
		FCanExecuteAction::CreateSP(this, &FSplineComponentVisualizer::AreMultipleKeysSelected));

	SplineComponentVisualizerActions->MapAction(
		Commands.SnapToLastSelectedZ,
		FExecuteAction::CreateSP(this, &FSplineComponentVisualizer::OnSnapSelectedToAxis, EAxis::Z),
		FCanExecuteAction::CreateSP(this, &FSplineComponentVisualizer::AreMultipleKeysSelected));

	SplineComponentVisualizerActions->MapAction(
		Commands.StraightenToNext,
		FExecuteAction::CreateSP(this, &FSplineComponentVisualizer::OnStraightenKey, 1),
		FCanExecuteAction::CreateSP(this, &FSplineComponentVisualizer::IsKeySelectionValid));

	SplineComponentVisualizerActions->MapAction(
		Commands.StraightenToPrevious,
		FExecuteAction::CreateSP(this, &FSplineComponentVisualizer::OnStraightenKey, -1),
		FCanExecuteAction::CreateSP(this, &FSplineComponentVisualizer::IsKeySelectionValid));

	SplineComponentVisualizerActions->MapAction(
		Commands.SetLockedAxisNone,
		FExecuteAction::CreateSP(this, &FSplineComponentVisualizer::OnLockAxis, EAxis::None),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FSplineComponentVisualizer::IsLockAxisSet, EAxis::None));

	SplineComponentVisualizerActions->MapAction(
		Commands.SetLockedAxisX,
		FExecuteAction::CreateSP(this, &FSplineComponentVisualizer::OnLockAxis, EAxis::X),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FSplineComponentVisualizer::IsLockAxisSet, EAxis::X));

	SplineComponentVisualizerActions->MapAction(
		Commands.SetLockedAxisY,
		FExecuteAction::CreateSP(this, &FSplineComponentVisualizer::OnLockAxis, EAxis::Y),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FSplineComponentVisualizer::IsLockAxisSet, EAxis::Y));

	SplineComponentVisualizerActions->MapAction(
		Commands.SetLockedAxisZ,
		FExecuteAction::CreateSP(this, &FSplineComponentVisualizer::OnLockAxis, EAxis::Z),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FSplineComponentVisualizer::IsLockAxisSet, EAxis::Z));

	SplineComponentVisualizerActions->MapAction(
		Commands.VisualizeRollAndScale,
		FExecuteAction::CreateSP(this, &FSplineComponentVisualizer::OnSetVisualizeRollAndScale),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FSplineComponentVisualizer::IsVisualizingRollAndScale));

	SplineComponentVisualizerActions->MapAction(
		Commands.DiscontinuousSpline,
		FExecuteAction::CreateSP(this, &FSplineComponentVisualizer::OnSetDiscontinuousSpline),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FSplineComponentVisualizer::IsDiscontinuousSpline));

	SplineComponentVisualizerActions->MapAction(
		Commands.ToggleClosedLoop,
		FExecuteAction::CreateSP(this, &FSplineComponentVisualizer::OnToggleClosedLoop),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FSplineComponentVisualizer::IsClosedLoop));

	SplineComponentVisualizerActions->MapAction(
		Commands.ResetToDefault,
		FExecuteAction::CreateSP(this, &FSplineComponentVisualizer::OnResetToDefault),
		FCanExecuteAction::CreateSP(this, &FSplineComponentVisualizer::CanResetToDefault));

	SplineComponentVisualizerActions->MapAction(
		Commands.AddAttributeKey,
		FExecuteAction::CreateSP(this, &FSplineComponentVisualizer::OnAddAttributeKey),
		FCanExecuteAction::CreateSP(this, &FSplineComponentVisualizer::CanAddAttributeKey));
	
	SplineComponentVisualizerActions->MapAction(
		Commands.DeleteAttributeKey,
		FExecuteAction::CreateSP(this, &FSplineComponentVisualizer::OnDeleteAttributeKey),
		FCanExecuteAction::CreateSP(this, &FSplineComponentVisualizer::CanDeleteAttributeKey));
	
	bool bAlign = false;
	bool bUseLineTrace = false;
	bool bUseBounds = false;
	bool bUsePivot = false;
	SplineComponentVisualizerActions->MapAction(
		FLevelEditorCommands::Get().SnapToFloor,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::SnapToFloor_Clicked, bAlign, bUseLineTrace, bUseBounds, bUsePivot),
		FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::ActorSelected_CanExecute)
		);

	bAlign = true;
	bUseLineTrace = false;
	bUseBounds = false;
	bUsePivot = false;
	SplineComponentVisualizerActions->MapAction(
		FLevelEditorCommands::Get().AlignToFloor,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::SnapToFloor_Clicked, bAlign, bUseLineTrace, bUseBounds, bUsePivot),
		FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::ActorSelected_CanExecute)
		);
}

bool FSplineComponentVisualizer::ShouldShowForSelectedSubcomponents(const UActorComponent* Component)
{
	if (const USplineComponent* SplineComp = Cast<const USplineComponent>(Component))
	{
		return SplineComp->bDrawDebug;
	}

	return false;
}

FSplineComponentVisualizer::~FSplineComponentVisualizer()
{
	FCoreUObjectDelegates::OnObjectTransacted.Remove(OnObjectTransactedHandle);
	OnObjectTransactedHandle.Reset();

	IModularFeatures::Get().UnregisterModularFeature(ISplineDetailsProvider::GetModularFeatureName(), this);
	FSplineComponentVisualizerCommands::Unregister();
}

void FSplineComponentVisualizer::AddReferencedObjects(FReferenceCollector& Collector)
{
	if (SelectionState)
	{
		Collector.AddReferencedObject(SelectionState);
	}
}

void FSplineComponentVisualizer::DirtyPDICache(const USplineComponent* SplineComp)
{
	FPDICache& ComponentPDICache = PDICache.FindOrAdd(SplineComp);
	ComponentPDICache.bDirty = true;
}

void FSplineComponentVisualizer::UpdatePDICache(const USplineComponent* SplineComp)
{
	FPDICache& ComponentPDICache = PDICache.FindOrAdd(SplineComp);

	if (!(SplineComponentVisualizerLocals::GRebuildPDICacheEveryFrame || ComponentPDICache.bDirty))
	{
		return;
	}
	
	FPDICache::FLineBatch LineBatch;
	FPDICache::FPointBatch PointBatch;
	ComponentPDICache.Reset();
	
	const FInterpCurveVector& SplineInfo = SplineComp->GetSplinePointsPosition();
	const USplineComponent* EditedSplineComp = GetEditedSplineComponent();

	const bool bIsSplineEditable = !SplineComp->bModifiedByConstructionScript; // bSplineHasBeenEdited || SplineInfo == Archetype->SplineCurves.Position || SplineComp->bInputSplinePointsToConstructionScript;

	constexpr FColor ReadOnlyColor = FColor(255, 0, 255, 255);
	const FColor NormalColor = bIsSplineEditable ? FColor(SplineComp->EditorUnselectedSplineSegmentColor.ToFColor(true)) : ReadOnlyColor;
	const FColor SelectedColor = bIsSplineEditable ? FColor(SplineComp->EditorSelectedSplineSegmentColor.ToFColor(true)) : ReadOnlyColor;
	const FColor TangentColor = bIsSplineEditable ? FColor(SplineComp->EditorTangentColor.ToFColor(true)) : ReadOnlyColor;
	const float GrabHandleSize = 10.0f + (bIsSplineEditable ? GetDefault<ULevelEditorViewportSettings>()->SelectedSplinePointSizeAdjustment : 0.0f);

	// Draw the tangent handles before anything else so they will not overdraw the rest of the spline
	if (SplineComp == EditedSplineComp)
	{
		check(SelectionState);

		if (SplineComp->GetNumberOfSplinePoints() == 0 && SelectionState->GetSelectedKeys().Num() > 0)
		{
			ChangeSelectionState(INDEX_NONE, false);
		}
		else
		{
			for (const TSet<int32> SelectedKeysCopy = SelectionState->GetSelectedKeys();
				int32 SelectedKey : SelectedKeysCopy)
			{
				check(SelectedKey >= 0);
				if (SelectedKey >= SplineComp->GetNumberOfSplinePoints())
				{
					// Catch any keys that might not exist anymore due to the underlying component changing.
					ChangeSelectionState(SelectedKey, true);
					continue;
				}

				if (SplineInfo.Points[SelectedKey].IsCurveKey())
				{
					const float TangentHandleSize = 8.0f + (bIsSplineEditable ? GetDefault<ULevelEditorViewportSettings>()->SplineTangentHandleSizeAdjustment : 0.0f);
					const float TangentScale = GetDefault<ULevelEditorViewportSettings>()->SplineTangentScale;

					const FVector Location = SplineComp->GetLocationAtSplinePoint(SelectedKey, ESplineCoordinateSpace::World);
					const FVector LeaveTangent = SplineComp->GetLeaveTangentAtSplinePoint(SelectedKey, ESplineCoordinateSpace::World) * TangentScale;
					const FVector ArriveTangent = SplineComp->GetArriveTangentAtSplinePoint(SelectedKey, ESplineCoordinateSpace::World) * TangentScale;
					
					// determine tangent coloration
					const bool bTangentSelected = (SelectedKey == SelectionState->GetSelectedTangentHandle());
					const ESelectedTangentHandle SelectedTangentHandleType = SelectionState->GetSelectedTangentHandleType();
					const bool bArriveSelected = bTangentSelected && (SelectedTangentHandleType == ESelectedTangentHandle::Arrive);
					const bool bLeaveSelected = bTangentSelected && (SelectedTangentHandleType == ESelectedTangentHandle::Leave);
					FColor ArriveColor = bArriveSelected ? SelectedColor : TangentColor;
					FColor LeaveColor = bLeaveSelected ? SelectedColor : TangentColor;

					LineBatch.Elements.Reset(2);
					LineBatch.Elements.Emplace(Location, Location - ArriveTangent, ArriveColor, SDPG_Foreground);
					LineBatch.Elements.Emplace(Location, Location + LeaveTangent, LeaveColor, SDPG_Foreground);
					ComponentPDICache.AddBatch(MoveTemp(LineBatch));
					
					PointBatch.Elements.Reset(1);
					if (bIsSplineEditable)
					{
						PointBatch.AllocHitProxyFunc = [SplineComp, SelectedKey]()
							{ return new HSplineTangentHandleProxy(SplineComp, SelectedKey, false); };
					}
					PointBatch.Elements.Emplace(Location + LeaveTangent, LeaveColor, TangentHandleSize, SDPG_Foreground);
					ComponentPDICache.AddBatch(MoveTemp(PointBatch));

					PointBatch.Elements.Reset(1);
					if (bIsSplineEditable)
					{
						PointBatch.AllocHitProxyFunc = [SplineComp, SelectedKey]()
							{ return new HSplineTangentHandleProxy(SplineComp, SelectedKey, true); };
					}
					PointBatch.Elements.Emplace(Location - ArriveTangent, ArriveColor, TangentHandleSize, SDPG_Foreground);
					ComponentPDICache.AddBatch(MoveTemp(PointBatch));
				}
			}
		}
	}

	const bool bShouldVisualizeScale = SplineComp->bShouldVisualizeScale;
	const float DefaultScale = SplineComp->ScaleVisualizationWidth;

	FVector OldKeyPos(0);
	FVector OldKeyRightVector(0);
	FVector OldKeyScale(0);

	const TSet<int32>& SelectedKeys = SelectionState->GetSelectedKeys();

	const int32 NumPoints = SplineInfo.Points.Num();
	const int32 NumSegments = SplineInfo.bIsLooped ? NumPoints : NumPoints - 1;
	for (int32 KeyIdx = 0; KeyIdx < NumSegments + 1; KeyIdx++)
	{
		const FVector NewKeyPos = SplineComp->GetLocationAtSplinePoint(KeyIdx, ESplineCoordinateSpace::World);
		const FVector NewKeyRightVector = SplineComp->GetRightVectorAtSplinePoint(KeyIdx, ESplineCoordinateSpace::World);
		const FVector NewKeyUpVector = SplineComp->GetUpVectorAtSplinePoint(KeyIdx, ESplineCoordinateSpace::World);
		const FVector NewKeyScale = SplineComp->GetScaleAtSplinePoint(KeyIdx) * DefaultScale;

		const FColor KeyColor = (SplineComp == EditedSplineComp && SelectedKeys.Contains(KeyIdx)) ? SelectedColor : NormalColor;

		// Draw the keypoint and up/right vectors
		if (KeyIdx < NumPoints)
		{
			if (bShouldVisualizeScale)
			{
				constexpr int32 ArcPoints = 20;

				LineBatch.Elements.Reset(3 + ArcPoints);
				LineBatch.Elements.Emplace(NewKeyPos, NewKeyPos - NewKeyRightVector * NewKeyScale.Y, KeyColor, SDPG_Foreground);
				LineBatch.Elements.Emplace(NewKeyPos, NewKeyPos + NewKeyRightVector * NewKeyScale.Y, KeyColor, SDPG_Foreground);
				LineBatch.Elements.Emplace(NewKeyPos, NewKeyPos + NewKeyUpVector * NewKeyScale.Z, KeyColor, SDPG_Foreground);
				FVector OldArcPos = NewKeyPos + NewKeyRightVector * NewKeyScale.Y;
				for (int32 ArcIndex = 1; ArcIndex <= ArcPoints; ArcIndex++)
				{
					float Sin;
					float Cos;
					FMath::SinCos(&Sin, &Cos, static_cast<float>(ArcIndex) * PI / static_cast<float>(ArcPoints));
					const FVector NewArcPos = NewKeyPos + Cos * NewKeyRightVector * NewKeyScale.Y + Sin * NewKeyUpVector * NewKeyScale.Z;
					LineBatch.Elements.Emplace(OldArcPos, NewArcPos, KeyColor, SDPG_Foreground);
					OldArcPos = NewArcPos;
				}
				ComponentPDICache.AddBatch(MoveTemp(LineBatch));
			}

			PointBatch.Elements.Reset(1);
			if (bIsSplineEditable)
			{
				PointBatch.AllocHitProxyFunc = [SplineComp, KeyIdx]()
					{  return new HSplineKeyProxy(SplineComp, KeyIdx); };
			}
			PointBatch.Elements.Emplace(NewKeyPos, KeyColor, GrabHandleSize, SDPG_Foreground);
			ComponentPDICache.AddBatch(MoveTemp(PointBatch));
		}

		// If not the first keypoint, draw a line to the previous keypoint.
		if (KeyIdx > 0)
		{
			UE::Geometry::FInterval1f SegmentParameterRange(SplineComp->GetInputKeyValueAtSplinePoint(KeyIdx - 1), SplineComp->GetInputKeyValueAtSplinePoint(KeyIdx));
			
			// how many?
			LineBatch.Elements.Reset();
			
			const FColor LineColor = NormalColor;
			if (bIsSplineEditable)
			{
				LineBatch.AllocHitProxyFunc = [SplineComp, KeyIdx]()
					{ return new HSplineSegmentProxy(SplineComp, KeyIdx - 1); };
			}

			const float SegmentLineThickness = GetDefault<ULevelEditorViewportSettings>()->SplineLineThicknessAdjustment;
			constexpr int32 NumStepsPerSegment = 21;
			
			// For constant interpolation - don't draw ticks - just draw dotted line.
			if (SplineInfo.Points[KeyIdx - 1].InterpMode == CIM_Constant)
			{
				FVector OldPos = OldKeyPos;
						
				const FVector SegmentVector = SplineComp->GetLocationAtSplinePoint(KeyIdx, ESplineCoordinateSpace::World) - OldKeyPos;
				const float SegmentLength = static_cast<float>(SegmentVector.Length());
				const FVector SegmentDirection = SegmentVector.GetSafeNormal();
				const float StepLength = SegmentLength / NumStepsPerSegment;
				const FVector SegmentStep = SegmentDirection * StepLength;
						
				for (int32 StepIdx = 1; StepIdx <= NumStepsPerSegment; StepIdx++)
				{
					const FVector NewPos = OldPos + SegmentStep;
					
					if (StepIdx % 2 == 1)
					{
						LineBatch.Elements.Emplace(OldPos, NewPos, LineColor, SDPG_World, SegmentLineThickness);
					}

					OldPos = NewPos;
				}
			}
			else
			{
				// Determine the colors to use
				const bool bIsEdited = (SplineComp == EditedSplineComp);
				const bool bKeyIdxLooped = (SplineInfo.bIsLooped && KeyIdx == NumPoints);
				const int32 BeginIdx = bKeyIdxLooped ? 0 : KeyIdx;
				const int32 EndIdx = KeyIdx - 1;
				const bool bBeginSelected = SelectedKeys.Contains(BeginIdx);
				const bool bEndSelected = SelectedKeys.Contains(EndIdx);
				const FColor BeginColor = bIsEdited && bBeginSelected ? SelectedColor : NormalColor;
				const FColor EndColor = bIsEdited && bEndSelected ? SelectedColor : NormalColor;
				
				// Find position on first keyframe.
				FVector OldPos = OldKeyPos;
				FVector OldRightVector = OldKeyRightVector;
				FVector OldScale = OldKeyScale;

				// Then draw a line for each substep.
				constexpr float PartialGradientProportion = 0.75f;
				constexpr int32 PartialNumSteps = static_cast<int32>(NumStepsPerSegment * PartialGradientProportion);

				for (int32 StepIdx = 1; StepIdx <= NumStepsPerSegment; StepIdx++)
				{
					const float StepRatio = static_cast<float>(StepIdx) / NumStepsPerSegment;
					const float Key = SegmentParameterRange.Interpolate(StepRatio);
					const FVector NewPos = SplineComp->GetLocationAtSplineInputKey(Key, ESplineCoordinateSpace::World);
					const FVector NewRightVector = SplineComp->GetRightVectorAtSplineInputKey(Key, ESplineCoordinateSpace::World);
					const FVector NewScale = SplineComp->GetScaleAtSplineInputKey(Key) * DefaultScale;

					// creates a gradient that starts partway through the selection
					FColor StepColor;
					if (bBeginSelected == bEndSelected)
					{
						StepColor = BeginColor;
					}
					else if (bBeginSelected && StepIdx > (NumStepsPerSegment - PartialNumSteps))
					{
						const float LerpRatio = (1.0f - StepRatio) / PartialGradientProportion;
						StepColor = FMath::Lerp(BeginColor.ReinterpretAsLinear(), EndColor.ReinterpretAsLinear(), LerpRatio).ToFColor(false);
					}
					else if (bEndSelected && StepIdx <= PartialNumSteps)
					{
						const float LerpRatio = 1.0f - (StepRatio / PartialGradientProportion);
						StepColor = FMath::Lerp(BeginColor.ReinterpretAsLinear(), EndColor.ReinterpretAsLinear(), LerpRatio).ToFColor(false);
					}
					else
					{
						StepColor = NormalColor; // unselected
					}

					LineBatch.Elements.Emplace(OldPos, NewPos, StepColor, SDPG_Foreground, SegmentLineThickness);
					if (bShouldVisualizeScale)
					{
						LineBatch.Elements.Emplace(OldPos - OldRightVector * OldScale.Y, NewPos - NewRightVector * NewScale.Y, LineColor, SDPG_Foreground);
						LineBatch.Elements.Emplace(OldPos + OldRightVector * OldScale.Y, NewPos + NewRightVector * NewScale.Y, LineColor, SDPG_Foreground);

						#if VISUALIZE_SPLINE_UPVECTORS
						const FVector NewUpVector = SplineComp->GetUpVectorAtSplineInputKey(Key, ESplineCoordinateSpace::World);
						LineBatch.Elements.Emplace(NewPos, NewPos + NewUpVector * SplineComp->ScaleVisualizationWidth * 0.5f, LineColor, SDPG_Foreground);
						LineBatch.Elements.Emplace(NewPos, NewPos + NewRightVector * SplineComp->ScaleVisualizationWidth * 0.5f, LineColor, SDPG_Foreground);
						#endif
					}

					OldPos = NewPos;
					OldRightVector = NewRightVector;
					OldScale = NewScale;
				}
			}

			ComponentPDICache.AddBatch(MoveTemp(LineBatch));
		}

		OldKeyPos = NewKeyPos;
		OldKeyRightVector = NewKeyRightVector;
		OldKeyScale = NewKeyScale;
	}

	// Render the selected attribute channel's values.
	if (FName SelectedAttributeName = SelectionState->GetSelectedAttributeName(); SelectedAttributeName != NAME_None)
	{
		for (int Idx = 0; Idx < SplineComp->GetNumberOfPropertyValues(SelectedAttributeName); ++Idx)
		{
			PointBatch.Elements.Reset(1);

			PointBatch.AllocHitProxyFunc = [SplineComp, Idx]()
				{ return new HSplineAttributeKeyProxy(SplineComp, Idx); };
			
			float Parameter = SplineComp->GetFloatPropertyInputKeyAtIndex(Idx, SelectedAttributeName);
			FVector PositionOnCurve = SplineComp->GetLocationAtSplineInputKey(Parameter, ESplineCoordinateSpace::World);
			PointBatch.Elements.Emplace(PositionOnCurve, FColor::Red, GrabHandleSize, SDPG_Foreground);

			ComponentPDICache.AddBatch(MoveTemp(PointBatch));
		}
	}

	ComponentPDICache.bDirty = false;
}

void FSplineComponentVisualizer::DrawVisualization(const UActorComponent* Component, const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	// TODO: DROP SUBDIVISION COUNT DURING INTERACTIVE CHANGES TO REDUCE RESAMPLE TIME AND GIVE AN IDEA OF THE LOOK
	
	QUICK_SCOPE_CYCLE_COUNTER(STAT_SplineComponentVisualizer_DrawVisualization);
	
	if (const USplineComponent* SplineComp = Cast<const USplineComponent>(Component))
	{
		if (!SplineComponentVisualizerLocals::IsEnabledForSpline(SplineComp))
		{
			return;
		}

		// If the component is not in the cache, this is our first time seeing it.
		if (!PDICache.Find(SplineComp))
		{
			// Not ideal to const_cast here, but 
			USplineComponent* MutableSplineComponent = const_cast<USplineComponent*>(SplineComp);

			// We assume it is safe to capture 'this' and 'MutableSplineComponent' because the lambda's user object will be 'this' and the delegates
			// are owned by the component itself (and therefore cannot be broadcast if the component is not valid).
			auto InvalidateCacheFunc = [this, MutableSplineComponent]() { DirtyPDICache(MutableSplineComponent); };
			
			// We don't retain the handles so lets just make sure we don't double register if our cache got cleared.
			MutableSplineComponent->TransformUpdated.RemoveAll(this);
			MutableSplineComponent->GetOnSplineChanged().RemoveAll(this);
			MutableSplineComponent->GetOnSplineUpdated().RemoveAll(this);
			MutableSplineComponent->GetOnSplineDisplayChanged().RemoveAll(this);

			MutableSplineComponent->TransformUpdated.AddSPLambda(this, [InvalidateCacheFunc](USceneComponent*, EUpdateTransformFlags, ETeleportType) { InvalidateCacheFunc(); });
			MutableSplineComponent->GetOnSplineChanged().AddSPLambda(this, InvalidateCacheFunc);
			MutableSplineComponent->GetOnSplineUpdated().AddSPLambda(this, InvalidateCacheFunc);
			MutableSplineComponent->GetOnSplineDisplayChanged().AddSPLambda(this, InvalidateCacheFunc);
		}

		// This does nothing if the cache is not dirty
		UpdatePDICache(SplineComp);

		if (const FPDICache* ComponentCache = PDICache.Find(SplineComp))
		{
			ComponentCache->Draw(PDI);
		}
	}
}

void FSplineComponentVisualizer::DrawVisualizationHUD(const UActorComponent* Component, const FViewport* Viewport, const FSceneView* View, FCanvas* Canvas) 
{
	if (const USplineComponent* SplineComp = Cast<const USplineComponent>(Component))
	{
		if (!SplineComponentVisualizerLocals::IsEnabledForSpline(SplineComp))
		{
			return;
		}
		
		const bool bIsSplineEditable = !SplineComp->bModifiedByConstructionScript; // bSplineHasBeenEdited || SplineInfo == Archetype->SplineCurves.Position || SplineComp->bInputSplinePointsToConstructionScript;
		const USplineComponent* EditedSplineComp = GetEditedSplineComponent();

		if (SplineComp == EditedSplineComp)
		{
			const FIntRect CanvasRect = Canvas->GetViewRect();

			auto DrawText = [&](const FText& SnapHelpText, float YOffset = 50.f)
			{
				int32 XL;
				int32 YL;
				StringSize(GEngine->GetLargeFont(), XL, YL, *SnapHelpText.ToString());
				const float DrawPositionX = FMath::FloorToFloat(static_cast<float>(CanvasRect.Min.X) + static_cast<float>(CanvasRect.Width() - XL) * 0.5f);
				const float DrawPositionY = static_cast<float>(CanvasRect.Min.Y) + YOffset;
				Canvas->DrawShadowedString(DrawPositionX, DrawPositionY, *SnapHelpText.ToString(), GEngine->GetLargeFont(), FLinearColor::Yellow);
			};
			
			if (bIsSnappingToActor)
			{
				static const FText SnapToActorHelp = LOCTEXT("SplinePointSnapToActorMessage", "Snap to Actor: Use Ctrl-LMB to select actor to use as target.");
				static const FText AlignToActorHelp = LOCTEXT("SplinePointAlignToActorMessage", "Snap Align to Actor: Use Ctrl-LMB to select actor to use as target.");
				static const FText AlignPerpToActorHelp = LOCTEXT("SplinePointAlignPerpToActorMessage", "Snap Align Perpendicular to Actor: Use Ctrl-LMB to select actor to use as target.");

				if (SnapToActorMode == ESplineComponentSnapMode::Snap)
				{
					DrawText(SnapToActorHelp);
				}
				else if (SnapToActorMode == ESplineComponentSnapMode::AlignToTangent)
				{
					DrawText(AlignToActorHelp);
				}
				else
				{
					DrawText(AlignPerpToActorHelp);
				}
			}
		}
		else
		{
			ResetTempModes();
		}
	}
}

void FSplineComponentVisualizer::ChangeSelectionState(int32 Index, bool bIsCtrlHeld)
{
	check(SelectionState);
	SelectionState->Modify();

	if (Index != INDEX_NONE)
	{
		// disallow selecting keys and attribute keys at the same time
		SelectionState->SetSelectedAttribute(INDEX_NONE, SelectionState->GetSelectedAttributeName());
	}
	
	TSet<int32>& SelectedKeys = SelectionState->ModifySelectedKeys();
	if (Index == INDEX_NONE)
	{
		SelectedKeys.Empty();
		SelectionState->SetLastKeyIndexSelected(INDEX_NONE);
	}
	else if (!bIsCtrlHeld)
	{
		SelectedKeys.Empty();
		SelectedKeys.Add(Index);
		SelectionState->SetLastKeyIndexSelected(Index);
	}
	else
	{
		// Add or remove from selection if Ctrl is held
		if (SelectedKeys.Contains(Index))
		{
			// If already in selection, toggle it off
			SelectedKeys.Remove(Index);

			if (SelectionState->GetLastKeyIndexSelected() == Index)
			{
				if (SelectedKeys.Num() == 0)
				{
					// Last key selected: clear last key index selected
					SelectionState->SetLastKeyIndexSelected(INDEX_NONE);
				}
				else
				{
					// Arbitarily set last key index selected to first member of the set (so that it is valid)
					SelectionState->SetLastKeyIndexSelected(*SelectedKeys.CreateConstIterator());
				}
			}
		}
		else
		{
			// Add to selection
			SelectedKeys.Add(Index);
			SelectionState->SetLastKeyIndexSelected(Index);
		}
	}

	if (SplineGeneratorPanel.IsValid())
	{
		SplineGeneratorPanel->OnSelectionUpdated();
	}

	if (Index != INDEX_NONE)
	{
		if (!DeselectedInEditorDelegateHandle.IsValid())
		{
			DeselectedInEditorDelegateHandle = GetEditedSplineComponent()->OnDeselectedInEditor.AddRaw(this, &FSplineComponentVisualizer::OnDeselectedInEditor);
		}
	}

	if (const USplineComponent* SplineComp = GetEditedSplineComponent())
	{
		DirtyPDICache(SplineComp);
	}
}


const USplineComponent* FSplineComponentVisualizer::UpdateSelectedSplineComponent(HComponentVisProxy* VisProxy)
{
	check(SelectionState);

	const USplineComponent* SplineComp = CastChecked<const USplineComponent>(VisProxy->Component.Get());

	DirtyPDICache(SplineComp);
	
	AActor* OldSplineOwningActor = SelectionState->GetSplinePropertyPath().GetParentOwningActor();
	FComponentPropertyPath NewSplinePropertyPath(SplineComp);
	SelectionState->SetSplinePropertyPath(NewSplinePropertyPath);
	AActor* NewSplineOwningActor = NewSplinePropertyPath.GetParentOwningActor();

	if (NewSplinePropertyPath.IsValid())
	{
		if (OldSplineOwningActor != NewSplineOwningActor)
		{
			// Reset selection state if we are selecting a different actor to the one previously selected
			ChangeSelectionState(INDEX_NONE, false);
			SelectionState->ClearSelectedSegmentIndex();
			SelectionState->ClearSelectedTangentHandle();
		}

		return SplineComp;
	}

	SelectionState->SetSplinePropertyPath(FComponentPropertyPath());
	return nullptr;
}

bool FSplineComponentVisualizer::VisProxyHandleClick(FEditorViewportClient* InViewportClient, HComponentVisProxy* VisProxy, const FViewportClick& Click)
{
	ResetTempModes();

	bool bVisProxyClickHandled = false;

	if(VisProxy && VisProxy->Component.IsValid())
	{
		check(SelectionState);

		if (const USplineComponent* SplineComp = CastChecked<const USplineComponent>(VisProxy->Component.Get());
			!SplineComp || (SplineComp && !SplineComponentVisualizerLocals::IsEnabledForSpline(SplineComp)))
		{
			// Consume no input if this visualizer is currently being suppressed.
			return false;
		}
		
		if (VisProxy->IsA(HSplineKeyProxy::StaticGetType()))
		{
			// Control point clicked
			const FScopedTransaction Transaction(LOCTEXT("SelectSplinePoint", "Select Spline Point"));
				 
			SelectionState->Modify();

			ResetTempModes();

			if (const USplineComponent* SplineComp = UpdateSelectedSplineComponent(VisProxy))
			{
				HSplineKeyProxy* KeyProxy = (HSplineKeyProxy*)VisProxy;

				// Modify the selection state, unless right-clicking on an already selected key
				const TSet<int32>& SelectedKeys = SelectionState->GetSelectedKeys();
				if (Click.GetKey() != EKeys::RightMouseButton || !SelectedKeys.Contains(KeyProxy->KeyIndex))
				{
					ChangeSelectionState(KeyProxy->KeyIndex, InViewportClient->IsCtrlPressed());
				}
				SelectionState->ClearSelectedSegmentIndex();
				SelectionState->ClearSelectedTangentHandle();

				if (SelectionState->GetLastKeyIndexSelected() == INDEX_NONE)
				{
					SelectionState->SetSplinePropertyPath(FComponentPropertyPath());
					return false;
				}

				SelectionState->SetCachedRotation(SplineComp->GetQuaternionAtSplinePoint(SelectionState->GetLastKeyIndexSelected(), ESplineCoordinateSpace::World));

				bVisProxyClickHandled = true;
			}
		}
		else if (VisProxy->IsA(HSplineSegmentProxy::StaticGetType()))
		{
			// Spline segment clicked
			const FScopedTransaction Transaction(LOCTEXT("SelectSplineSegment", "Select Spline Segment"));

			SelectionState->Modify();

			ResetTempModes();

			if (const USplineComponent* SplineComp = UpdateSelectedSplineComponent(VisProxy))
			{
				// Divide segment into subsegments and test each subsegment against ray representing click position and camera direction.
				// Closest encounter with the spline determines the spline position.
				const int32 NumSubdivisions = 16;

				HSplineSegmentProxy* SegmentProxy = (HSplineSegmentProxy*)VisProxy;

				// Ignore Ctrl key, segments should only be selected one at time
				ChangeSelectionState(SegmentProxy->SegmentIndex, false);
				SelectionState->SetSelectedSegmentIndex(SegmentProxy->SegmentIndex);
				SelectionState->ClearSelectedTangentHandle();

				if (SelectionState->GetLastKeyIndexSelected() == INDEX_NONE)
				{
					SelectionState->SetSplinePropertyPath(FComponentPropertyPath());
					return false;
				}

				SelectionState->SetCachedRotation(SplineComp->GetQuaternionAtSplinePoint(SelectionState->GetLastKeyIndexSelected(), ESplineCoordinateSpace::World));

				int32 SelectedSegmentIndex = SelectionState->GetSelectedSegmentIndex();
				float SubsegmentStartKey = static_cast<float>(SelectedSegmentIndex);
				FVector SubsegmentStart = SplineComp->GetLocationAtSplineInputKey(SubsegmentStartKey, ESplineCoordinateSpace::World);

				double ClosestDistance = TNumericLimits<double>::Max();
				FVector BestLocation = SubsegmentStart;

				for (int32 Step = 1; Step < NumSubdivisions; Step++)
				{
					const float SubsegmentEndKey = static_cast<float>(SelectedSegmentIndex) + static_cast<float>(Step) / static_cast<float>(NumSubdivisions);
					const FVector SubsegmentEnd = SplineComp->GetLocationAtSplineInputKey(SubsegmentEndKey, ESplineCoordinateSpace::World);

					FVector SplineClosest;
					FVector RayClosest;
					FMath::SegmentDistToSegmentSafe(SubsegmentStart, SubsegmentEnd, Click.GetOrigin(), Click.GetOrigin() + Click.GetDirection() * 50000.0f, SplineClosest, RayClosest);

					const double Distance = FVector::DistSquared(SplineClosest, RayClosest);
					if (Distance < ClosestDistance)
					{
						ClosestDistance = Distance;
						BestLocation = SplineClosest;
					}

					SubsegmentStartKey = SubsegmentEndKey;
					SubsegmentStart = SubsegmentEnd;
				}

				SelectionState->SetSelectedSplinePosition(BestLocation);

				bVisProxyClickHandled = true;
			}
		}
		else if (VisProxy->IsA(HSplineTangentHandleProxy::StaticGetType()))
		{
			// Spline segment clicked
			const FScopedTransaction Transaction(LOCTEXT("SelectSplineSegment", "Select Spline Segment"));

			SelectionState->Modify();

			ResetTempModes();

			if (const USplineComponent* SplineComp = UpdateSelectedSplineComponent(VisProxy))
			{
				// Tangent handle clicked

				HSplineTangentHandleProxy* KeyProxy = (HSplineTangentHandleProxy*)VisProxy;

				// Note: don't change key selection when a tangent handle is clicked.
				// Ignore Ctrl-modifier, cannot select multiple tangent handles at once.
				// To do: replace the following section with new method ClearMetadataSelectionState()
				// since this is the only reason ChangeSelectionState is being called here.
				TSet<int32> SelectedKeysCopy(SelectionState->GetSelectedKeys());
				ChangeSelectionState(KeyProxy->KeyIndex, false);
				TSet<int32>& SelectedKeys = SelectionState->ModifySelectedKeys();
				for (int32 KeyIndex : SelectedKeysCopy)
				{
					if (KeyIndex != KeyProxy->KeyIndex)
					{
						SelectedKeys.Add(KeyIndex);
					}
				}

				SelectionState->ClearSelectedSegmentIndex();
				SelectionState->SetSelectedTangentHandle(KeyProxy->KeyIndex);
				SelectionState->SetSelectedTangentHandleType(KeyProxy->bArriveTangent ? ESelectedTangentHandle::Arrive : ESelectedTangentHandle::Leave);
				SelectionState->SetCachedRotation(SplineComp->GetQuaternionAtSplinePoint(SelectionState->GetSelectedTangentHandle(), ESplineCoordinateSpace::World));

				bVisProxyClickHandled = true;
			}
		}
		else if (VisProxy->IsA(HSplineAttributeKeyProxy::StaticGetType()))
		{
			// Spline attribute key clicked
			const FScopedTransaction Transaction(LOCTEXT("SelectAttributeValue", "Select Attribute Value"));

			SelectionState->Modify();

			ResetTempModes();

			if (const USplineComponent* SplineComp = UpdateSelectedSplineComponent(VisProxy))
			{
				HSplineAttributeKeyProxy* AttributeKeyProxy = static_cast<HSplineAttributeKeyProxy*>(VisProxy);
				
				SelectionState->SetSelectedAttribute(AttributeKeyProxy->KeyIndex, SelectionState->GetSelectedAttributeName());
				const float InputKey = SplineComp->GetFloatPropertyInputKeyAtIndex(AttributeKeyProxy->KeyIndex, SelectionState->GetSelectedAttributeName());
				SelectionState->SetCachedRotation(SplineComp->GetQuaternionAtSplineInputKey(InputKey, ESplineCoordinateSpace::World));
				
				bVisProxyClickHandled = true;
			}
		}
	}

	if (bVisProxyClickHandled)
	{
		if (const USplineComponent* SplineComp = GetEditedSplineComponent())
		{
			DirtyPDICache(SplineComp);
		}
		
		GEditor->RedrawLevelEditingViewports(true);
	}

	return bVisProxyClickHandled;
}

void FSplineComponentVisualizer::SetEditedSplineComponent(const USplineComponent* InSplineComponent) 
{
	check(SelectionState);
	SelectionState->Modify();
	SelectionState->Reset();

	FComponentPropertyPath SplinePropertyPath(InSplineComponent);
	SelectionState->SetSplinePropertyPath(SplinePropertyPath);
	
	DirtyPDICache(InSplineComponent);
}

USplineComponent* FSplineComponentVisualizer::GetEditedSplineComponent() const
{
	check(SelectionState);
	return Cast<USplineComponent>(SelectionState->GetSplinePropertyPath().GetComponent());
}

UActorComponent* FSplineComponentVisualizer::GetEditedComponent() const
{
	return Cast<UActorComponent>(GetEditedSplineComponent());
}

bool FSplineComponentVisualizer::GetWidgetLocation(const FEditorViewportClient* ViewportClient, FVector& OutLocation) const
{
	USplineComponent* SplineComp = GetEditedSplineComponent();
	if (SplineComp != nullptr)
	{
		if (!SplineComponentVisualizerLocals::IsEnabledForSpline(SplineComp))
		{
			return false;
		}
		
		check(SelectionState);

		const int32 NumPoints = SplineComp->GetNumberOfSplinePoints();
		
		const TSet<int32>& SelectedKeys = SelectionState->GetSelectedKeys();
		int32 LastKeyIndexSelected = SelectionState->GetLastKeyIndexSelected();
		int32 SelectedAttributeIndex = SelectionState->GetSelectedAttributeIndex();

		int32 SelectedTangentHandle = SelectionState->GetSelectedTangentHandle();
		ESelectedTangentHandle SelectedTangentHandleType = SelectionState->GetSelectedTangentHandleType();
		if (SelectedTangentHandle != INDEX_NONE)
		{
			// If tangent handle index is set, use that
			check(SelectedTangentHandle < NumPoints);
			const auto& Point = SplineComp->GetLocationAtSplinePoint(SelectedTangentHandle, ESplineCoordinateSpace::Local);

			check(SelectedTangentHandleType != ESelectedTangentHandle::None);
			const float TangentScale = GetDefault<ULevelEditorViewportSettings>()->SplineTangentScale;

			if (SelectedTangentHandleType == ESelectedTangentHandle::Leave)
			{
				const FVector Tangent = SplineComp->GetLeaveTangentAtSplinePoint(SelectedTangentHandle, ESplineCoordinateSpace::Local);
				OutLocation = SplineComp->GetComponentTransform().TransformPosition(Point + Tangent * TangentScale);
			}
			else if (SelectedTangentHandleType == ESelectedTangentHandle::Arrive)
			{
				const FVector Tangent = SplineComp->GetArriveTangentAtSplinePoint(SelectedTangentHandle, ESplineCoordinateSpace::Local);
				OutLocation = SplineComp->GetComponentTransform().TransformPosition(Point - Tangent * TangentScale);
			}

			return true;
		}
		else if (SelectedAttributeIndex != INDEX_NONE)
		{
			// If we have a selected attribute value, use that.
			if (SelectedAttributeIndex >= 0 && SelectedAttributeIndex < SplineComp->GetNumberOfPropertyValues(SelectionState->GetSelectedAttributeName()))
			{
				const float Param = SplineComp->GetFloatPropertyInputKeyAtIndex(SelectedAttributeIndex, SelectionState->GetSelectedAttributeName());
				OutLocation = SplineComp->GetLocationAtSplineInputKey(Param, ESplineCoordinateSpace::World);
				return true;
			}
		}
		else if (LastKeyIndexSelected != INDEX_NONE)
		{
			// Otherwise use the last key index set
			check(LastKeyIndexSelected >= 0);
			if (LastKeyIndexSelected < NumPoints)
			{
				check(SelectedKeys.Contains(LastKeyIndexSelected));
				OutLocation = SplineComp->GetLocationAtSplinePoint(LastKeyIndexSelected, ESplineCoordinateSpace::World);
				if (!DuplicateDelayAccumulatedDrag.IsZero())
				{
					OutLocation += DuplicateDelayAccumulatedDrag;
				}
				return true;
			}
		}
	}

	return false;
}


bool FSplineComponentVisualizer::GetCustomInputCoordinateSystem(const FEditorViewportClient* ViewportClient, FMatrix& OutMatrix) const
{
	if (ViewportClient->GetWidgetCoordSystemSpace() == COORD_Local || ViewportClient->GetWidgetMode() == UE::Widget::WM_Rotate)
	{
		USplineComponent* SplineComp = GetEditedSplineComponent();
		if (SplineComp != nullptr)
		{
			if (!SplineComponentVisualizerLocals::IsEnabledForSpline(SplineComp))
			{
				return false;
			}
			
			check(SelectionState);
			OutMatrix = FRotationMatrix::Make(SelectionState->GetCachedRotation());
			return true;
		}
	}

	return false;
}


bool FSplineComponentVisualizer::IsVisualizingArchetype() const
{
	USplineComponent* SplineComp = GetEditedSplineComponent();
	return (SplineComp && SplineComp->GetOwner() && FActorEditorUtils::IsAPreviewOrInactiveActor(SplineComp->GetOwner()));
}

bool FSplineComponentVisualizer::IsAnySelectedKeyIndexOutOfRange(const USplineComponent* Comp) const
{
	const int32 NumPoints = Comp->GetSplinePointsPosition().Points.Num();
	check(SelectionState);
	const TSet<int32>& SelectedKeys = SelectionState->GetSelectedKeys();
	return Algo::AnyOf(SelectedKeys, [NumPoints](int32 Index) { return Index >= NumPoints; });
}

bool FSplineComponentVisualizer::IsSingleKeySelected() const
{
	USplineComponent* SplineComp = GetEditedSplineComponent();
	check(SelectionState);
	const TSet<int32>& SelectedKeys = SelectionState->GetSelectedKeys();
	int32 LastKeyIndexSelected = SelectionState->GetLastKeyIndexSelected();
	return (SplineComp != nullptr &&
		SelectedKeys.Num() == 1 &&
		LastKeyIndexSelected != INDEX_NONE);
}

bool FSplineComponentVisualizer::AreMultipleKeysSelected() const
{
	USplineComponent* SplineComp = GetEditedSplineComponent();
	check(SelectionState);
	const TSet<int32>& SelectedKeys = SelectionState->GetSelectedKeys();
	int32 LastKeyIndexSelected = SelectionState->GetLastKeyIndexSelected();
	return (SplineComp != nullptr &&
		SelectedKeys.Num() > 1 &&
		LastKeyIndexSelected != INDEX_NONE);
}

bool FSplineComponentVisualizer::AreKeysSelected() const
{
	return (IsSingleKeySelected() || AreMultipleKeysSelected());
}

bool FSplineComponentVisualizer::HandleInputDelta(FEditorViewportClient* ViewportClient, FViewport* Viewport, FVector& DeltaTranslateIn, FRotator& DeltaRotate, FVector& DeltaScale)
{
	using namespace SplineComponentVisualizerLocals;

	ResetTempModes();

	USplineComponent* SplineComp = GetEditedSplineComponent();
	if (!SplineComp || !SplineComponentVisualizerLocals::IsEnabledForSpline(SplineComp))
	{
		return false;
	}
	
	bool bInputHandled = false;
		
	if (IsAnySelectedKeyIndexOutOfRange(SplineComp))
	{
		// Something external has changed the number of spline points, meaning that the cached selected keys are no longer valid
		EndEditing();
		return false;
	}

	check(SelectionState);

	// Use a local value for DeltaTranslate so that we can modify it based on the SnapToSurface setting without
	// changing it for the caller (that parameter should probably be const, but it's a base class function)
	FVector DeltaTranslate = DeltaTranslateIn;

	// It's tough to port the actor "surface snapping" toggle behavior here because the original code only works on actors
	// and involves things like keeping track of last transform, snapping to surface even if moving in a different plane, etc
	// (see FLevelEditorViewportClient::ProjectSelectedElementsIntoWorld). However we can at least support the limited but useful case
	// of trying to drag points around by the ball of the widget.
	const ULevelEditorViewportSettings* ViewportSettings = GetDefault<ULevelEditorViewportSettings>();
	const float SnapOffsetExtent = (ViewportSettings->SnapToSurface.bEnabled) ? (ViewportSettings->SnapToSurface.SnapOffsetExtent) : (0.0f);
	if (ViewportSettings->SnapToSurface.bEnabled
		&& ViewportClient->GetWidgetMode() == UE::Widget::EWidgetMode::WM_Translate
		&& (ViewportClient->GetCurrentWidgetAxis() == EAxisList::Screen || ViewportClient->GetCurrentWidgetAxis() == EAxisList::XYZ)
		&& !ViewportClient->IsOrtho())
	{
		FHitResult HitResult;
		if (RaycastWorld(SplineComp->GetWorld(), ViewportClient, Viewport, HitResult))
		{
			FVector3d NewGizmoLocation = HitResult.ImpactPoint + ViewportSettings->SnapToSurface.SnapOffsetExtent * HitResult.ImpactNormal;
			DeltaTranslate = NewGizmoLocation - ViewportClient->GetWidgetLocation();
		}
	}

	if (SelectionState->GetSelectedAttributeIndex() != INDEX_NONE)
	{
		bInputHandled = TransformSelectedAttribute(EPropertyChangeType::Interactive, DeltaTranslate);
	}
	else if (SelectionState->GetSelectedTangentHandle() != INDEX_NONE)
	{
		// Transform the tangent using an EPropertyChangeType::Interactive change. Later on, at the end of mouse tracking, a non-interactive change will be notified via void TrackingStopped :
		bInputHandled = TransformSelectedTangent(EPropertyChangeType::Interactive, DeltaTranslate);
	}
	else if (ViewportClient->IsAltPressed())
	{
		if (ViewportClient->GetWidgetMode() == UE::Widget::WM_Translate && ViewportClient->GetCurrentWidgetAxis() != EAxisList::None && SelectionState->GetSelectedKeys().Num() == 1)
		{
			static const int MaxDuplicationDelay = 3;

			FVector Drag = DeltaTranslate;

			if (bAllowDuplication)
			{
				float SmallestGridSize = 1.0f;
				const TArray<float>& PosGridSizes = GEditor->GetCurrentPositionGridArray();
				if (PosGridSizes.IsValidIndex(0))
				{
					SmallestGridSize = PosGridSizes[0];
				}

				// When grid size is set to a value other than the smallest grid size, do not delay duplication
				if (DuplicateDelay >= MaxDuplicationDelay || GEditor->GetGridSize() > SmallestGridSize)
				{
					Drag += DuplicateDelayAccumulatedDrag;
					DuplicateDelayAccumulatedDrag = FVector::ZeroVector;

					bAllowDuplication = false;
					bDuplicatingSplineKey = true;

					DuplicateKeyForAltDrag(Drag);
				}
				else
				{ 
					DuplicateDelay++;
					DuplicateDelayAccumulatedDrag += DeltaTranslate;
				}
			}
			else
			{
				UpdateDuplicateKeyForAltDrag(Drag);
			}

			bInputHandled = true;
		}
	}
	else
	{
		const bool bHasLastKey = SelectionState->GetLastKeyIndexSelected() != INDEX_NONE;
		const bool bHasSelectedKeys = SelectionState->GetSelectedKeys().Num() > 0;
		if (bHasLastKey && bHasSelectedKeys)
		{
			// Transform the spline keys using an EPropertyChangeType::Interactive change. Later on, at the end of mouse tracking, a non-interactive change will be notified via void TrackingStopped :
			bInputHandled = TransformSelectedKeys(EPropertyChangeType::Interactive, DeltaTranslate, DeltaRotate, DeltaScale);
		}
		else if (bHasLastKey != bHasSelectedKeys)
		{
			// One half of the selection is set, the other isn't. Reconcile -- TransformSelectedKeys would assert.
			UE_LOG(LogSplineComponentVisualizer, Log, TEXT("Stale selection drift at TransformSelectedKeys gate -- clearing"));
			ChangeSelectionState(INDEX_NONE, false);
		}
	}
	
	if (bInputHandled)
	{
		if (AActor* OwnerActor = SplineComp->GetOwner())
		{
			OwnerActor->PostEditMove(false);
		}

		DirtyPDICache(SplineComp);
	}

	return bInputHandled;
}

bool FSplineComponentVisualizer::TransformSelectedTangent(EPropertyChangeType::Type InPropertyChangeType, const FVector& InDeltaTranslate)
{
	USplineComponent* SplineComp = GetEditedSplineComponent();
	if (SplineComp != nullptr)
	{
		check(SelectionState);
		int32 SelectedTangentHandle;
		ESelectedTangentHandle SelectedTangentHandleType;
		SelectionState->GetVerifiedSelectedTangentHandle(SplineComp->GetNumberOfSplinePoints(), SelectedTangentHandle, SelectedTangentHandleType);

		if (!InDeltaTranslate.IsZero())
		{
			SplineComp->Modify();

			const float TangentScale = GetDefault<ULevelEditorViewportSettings>()->SplineTangentScale;

			if (SplineComp->bAllowDiscontinuousSpline)
			{
				if (SelectedTangentHandleType == ESelectedTangentHandle::Leave)
				{
					const FVector ArriveTangent = SplineComp->GetArriveTangentAtSplinePoint(SelectedTangentHandle, ESplineCoordinateSpace::Local);
					const FVector LeaveTangent = SplineComp->GetLeaveTangentAtSplinePoint(SelectedTangentHandle, ESplineCoordinateSpace::Local) + SplineComp->GetComponentTransform().InverseTransformVector(InDeltaTranslate) / TangentScale;
					SplineComp->SetTangentsAtSplinePoint(SelectedTangentHandle, ArriveTangent, LeaveTangent, ESplineCoordinateSpace::Local, false);
				}
				else
				{
					const FVector ArriveTangent = SplineComp->GetArriveTangentAtSplinePoint(SelectedTangentHandle, ESplineCoordinateSpace::Local) + SplineComp->GetComponentTransform().InverseTransformVector(-InDeltaTranslate) / TangentScale;
					const FVector LeaveTangent = SplineComp->GetLeaveTangentAtSplinePoint(SelectedTangentHandle, ESplineCoordinateSpace::Local);
					SplineComp->SetTangentsAtSplinePoint(SelectedTangentHandle, ArriveTangent, LeaveTangent, ESplineCoordinateSpace::Local, false);
				}
			}
			else
			{
				const FVector Delta = (SelectedTangentHandleType == ESelectedTangentHandle::Leave) ? InDeltaTranslate : -InDeltaTranslate;
				const FVector Tangent = SplineComp->GetLeaveTangentAtSplinePoint(SelectedTangentHandle, ESplineCoordinateSpace::Local) + SplineComp->GetComponentTransform().InverseTransformVector(Delta) / TangentScale;
				SplineComp->SetTangentAtSplinePoint(SelectedTangentHandle, Tangent, ESplineCoordinateSpace::Local, false);
			}
		}

		SplineComp->UpdateSpline();
		SplineComp->bSplineHasBeenEdited = true;

		NotifyPropertiesModified(SplineComp, SplineProperties, InPropertyChangeType);

		return true;
	}

	return false;
}

bool FSplineComponentVisualizer::TransformSelectedKeys(EPropertyChangeType::Type InPropertyChangeType, const FVector& InDeltaTranslate, const FRotator& InDeltaRotate, const FVector& InDeltaScale)
{
	USplineComponent* SplineComp = GetEditedSplineComponent();
	if (SplineComp != nullptr)
	{
		const int32 NumPoints = SplineComp->GetNumberOfSplinePoints();

		check(SelectionState);
		const TSet<int32>& SelectedKeys = SelectionState->GetSelectedKeys();
		int32 LastKeyIndexSelected = SelectionState->GetVerifiedLastKeyIndexSelected(NumPoints);
		check(SelectedKeys.Num() > 0);
		check(SelectedKeys.Contains(LastKeyIndexSelected));

		SplineComp->Modify();

		for (int32 SelectedKeyIndex : SelectedKeys)
		{
			check(SelectedKeyIndex >= 0); 
			check(SelectedKeyIndex < NumPoints);

			if (!InDeltaTranslate.IsZero())
			{
				const FVector NewPosition = SplineComp->GetLocationAtSplinePoint(SelectedKeyIndex, ESplineCoordinateSpace::World) + InDeltaTranslate;
				SplineComp->SetLocationAtSplinePoint(SelectedKeyIndex, NewPosition, ESplineCoordinateSpace::World, false);
			}

			if (!InDeltaRotate.IsZero())
			{
				// note [nickolas.drake]: removed tangent setting code here because we set tangent in the SetRotation call below.

				// Rotate spline rotation according to delta rotation
				FQuat NewRot = SplineComp->GetQuaternionAtSplinePoint(SelectedKeyIndex, ESplineCoordinateSpace::World);
				NewRot = InDeltaRotate.Quaternion() * NewRot; // apply world-space rotation
				SplineComp->SetQuaternionAtSplinePoint(SelectedKeyIndex, NewRot, ESplineCoordinateSpace::World, false);
			}

			if (InDeltaScale.X != 0.0f)
			{
				const FVector NewTangent = SplineComp->GetLeaveTangentAtSplinePoint(SelectedKeyIndex, ESplineCoordinateSpace::Local) * (1.f + InDeltaScale.X);
				SplineComp->SetTangentsAtSplinePoint(SelectedKeyIndex, NewTangent, NewTangent, ESplineCoordinateSpace::Local, false);
			}

			if (InDeltaScale.Y != 0.0f)
			{
				// Scale in Y adjusts the scale spline
				FVector NewScale = SplineComp->GetScaleAtSplinePoint(SelectedKeyIndex);
				NewScale.Y *= (1.f + InDeltaScale.Y);
				SplineComp->SetScaleAtSplinePoint(SelectedKeyIndex, NewScale, false);
			}

			if (InDeltaScale.Z != 0.0f)
			{
				// Scale in Z adjusts the scale spline
				FVector NewScale = SplineComp->GetScaleAtSplinePoint(SelectedKeyIndex);
				NewScale.Z *= (1.f + InDeltaScale.Z);
				SplineComp->SetScaleAtSplinePoint(SelectedKeyIndex, NewScale, false);
			}
		}

		SplineComp->UpdateSpline();
		SplineComp->bSplineHasBeenEdited = true;

		NotifyPropertiesModified(SplineComp, SplineProperties, InPropertyChangeType);

		if (!InDeltaRotate.IsZero())
		{
			SelectionState->Modify();
			SelectionState->SetCachedRotation(SplineComp->GetQuaternionAtSplinePoint(LastKeyIndexSelected, ESplineCoordinateSpace::World));
		}

		GEditor->RedrawLevelEditingViewports(true);

		return true;
	}

	return false;
}

bool FSplineComponentVisualizer::TransformSelectedAttribute(EPropertyChangeType::Type InPropertyChangeType, const FVector& InDeltaTranslate)
{
	USplineComponent* SplineComp = GetEditedSplineComponent();
	if (SplineComp != nullptr)
	{
		check(SelectionState);
		const int32 SelectedAttributeIndex = SelectionState->GetSelectedAttributeIndex();
		const FName SelectedAttributeName = SelectionState->GetSelectedAttributeName();
		check(SelectedAttributeIndex != INDEX_NONE);
		check(SelectedAttributeName != NAME_None);

		SplineComp->Modify();
		if (AActor* Owner = SplineComp->GetOwner())
		{
			Owner->Modify();
		}
		
		const float CurrentParam = SplineComp->GetFloatPropertyInputKeyAtIndex(SelectedAttributeIndex, SelectedAttributeName);
		const FVector CurrentPosition = SplineComp->GetLocationAtSplineInputKey(CurrentParam, ESplineCoordinateSpace::World);
		const FVector NewPosition = CurrentPosition + InDeltaTranslate;

		const float NewParam = SplineComp->FindInputKeyClosestToWorldLocation(NewPosition);
		const int32 NewIndex = SplineComp->SetFloatPropertyInputKeyAtIndex(SelectedAttributeIndex, NewParam, SelectedAttributeName);

		if (NewIndex != SelectedAttributeIndex)
		{
			// Index was invalidated because we reordered the attribute storage, update selection and invalidate proxies
			SelectionState->SetSelectedAttribute(NewIndex, SelectionState->GetSelectedAttributeName());
			GEditor->RedrawAllViewports();
		}
		
		SelectionState->SetCachedRotation(SplineComp->GetQuaternionAtSplineInputKey(NewParam, ESplineCoordinateSpace::World));

		SplineComp->bSplineHasBeenEdited = true;
		NotifyPropertiesModified(SplineComp, SplineProperties, InPropertyChangeType);
		GEditor->RedrawLevelEditingViewports(true);

		return true;
	}

	return false;
}

bool FSplineComponentVisualizer::HandleInputKey(FEditorViewportClient* ViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event)
{
	bool bHandled = false;

	USplineComponent* SplineComp = GetEditedSplineComponent();
	if (SplineComp)
	{
		if (IsAnySelectedKeyIndexOutOfRange(SplineComp))
		{
			// Something external has changed the number of spline points, meaning that the cached selected keys are no longer valid
			EndEditing();
			return false;
		}

		if (!SplineComponentVisualizerLocals::IsEnabledForSpline(SplineComp))
		{
			return false;
		}
	}

	if (Key == EKeys::LeftMouseButton && Event == IE_Released)
	{
		if (SplineComp != nullptr)
		{
			check(SelectionState);

			// Recache widget rotation
			
			if (int32 AttributeIndex = SelectionState->GetSelectedAttributeIndex(); AttributeIndex != INDEX_NONE)
			{
				const float Param = SplineComp->GetFloatPropertyInputKeyAtIndex(AttributeIndex, SelectionState->GetSelectedAttributeName());
				
				SelectionState->Modify();
				SelectionState->SetCachedRotation(SplineComp->GetQuaternionAtSplineInputKey(Param, ESplineCoordinateSpace::World));
			}
			else
			{
				int32 Index = SelectionState->GetSelectedTangentHandle();
				if (Index == INDEX_NONE)
				{
					// If not set, fall back to last key index selected
					Index = SelectionState->GetLastKeyIndexSelected();
				}

				SelectionState->Modify();
				SelectionState->SetCachedRotation(SplineComp->GetQuaternionAtSplinePoint(Index, ESplineCoordinateSpace::World));
			}
		}

		// Reset duplication on LMB release
		ResetAllowDuplication();
	}

	if (Event == IE_Pressed)
	{
		bHandled = SplineComponentVisualizerActions->ProcessCommandBindings(Key, FSlateApplication::Get().GetModifierKeys(), false);
	}

	return bHandled;
}

bool FSplineComponentVisualizer::HandleModifiedClick(FEditorViewportClient* InViewportClient, HHitProxy* HitProxy, const FViewportClick& Click)
{
	if (const USplineComponent* SplineComp = GetEditedSplineComponent();
		SplineComp && !SplineComponentVisualizerLocals::IsEnabledForSpline(SplineComp))
	{
		return false;
	}
	
	if (Click.IsControlDown())
	{
		ESplineComponentSnapMode SnapMode;

		if (GetSnapToActorMode(SnapMode))
		{
			ResetTempModes();

			if (HitProxy && HitProxy->IsA(HActor::StaticGetType()))
			{
				HActor* ActorProxy = static_cast<HActor*>(HitProxy);
				SnapKeyToActor(ActorProxy->Actor, SnapMode);
			}

			return true;
		}

	}

	ResetTempModes();

	/*
	if (Click.IsControlDown())
	{
		// Add points on Ctrl-Click if the last spline point is selected.

		USplineComponent* SplineComp = GetEditedSplineComponent();
		if (SplineComp != nullptr)
		{
			FInterpCurveVector& SplinePosition = SplineComp->GetSplinePointsPosition();
			int32 NumPoints = SplinePosition.Points.Num();

			// to do add end point
			check(SelectionState);
			const TSet<int32>& SelectedKeys = SelectionState->GetSelectedKeys();
			if (SelectedKeys.Num() == 1 && !SplineComp->IsClosedLoop())
			{
				int32 LastKeyIndexSelected = SelectionState->GetLastKeyIndexSelected();
				check(LastKeyIndexSelected != INDEX_NONE);
				check(SelectedKeys.Contains(LastKeyIndexSelected));

				if (LastKeyIndexSelected == 0)
				{
					int32 KeyIdx = LastKeyIndexSelected;

					FInterpCurvePoint<FVector>& EditedPoint = SplinePosition.Points[LastKeyIndexSelected];

					FHitResult Hit(1.0f);
					FCollisionQueryParams Params(SCENE_QUERY_STAT(MoveSplineKeyToTrace), true);

					// Find key position in world space
					const FVector CurrentWorldPos = SplineComp->GetComponentTransform().TransformPosition(EditedPoint.OutVal);

					FVector DeltaTranslate = FVector::ZeroVector;

					if (SplineComp->GetWorld()->LineTraceSingleByChannel(Hit, Click.GetOrigin(), Click.GetOrigin() + Click.GetDirection() * WORLD_MAX, ECC_WorldStatic, Params))
					{
						DeltaTranslate = Hit.Location - CurrentWorldPos;
					}
					else
					{
						FVector ArriveTangent = SplineComp->GetComponentTransform().GetRotation().RotateVector(EditedPoint.ArriveTangent); // convert local-space tangent vector to world-space
						DeltaTranslate = ArriveTangent.GetSafeNormal() * ArriveTangent.Size() * 0.5;
						DeltaTranslate = ArriveTangent.GetSafeNormal() * ArriveTangent.Size() * 0.5;
					}

					OnAddKey();
					TransformSelectedKeys(DeltaTranslate);

					return true;
				}
			}
		}
	}
	*/
	return false;
}


bool FSplineComponentVisualizer::HandleBoxSelect(const FBox& InBox, FEditorViewportClient* InViewportClient, FViewport* InViewport) 
{
	const FScopedTransaction Transaction(LOCTEXT("HandleBoxSelect", "Box Select Spline Points"));

	check(SelectionState);
	SelectionState->Modify();

	ResetTempModes();

	USplineComponent* SplineComp = GetEditedSplineComponent();
	if (SplineComp != nullptr)
	{
		if (!SplineComponentVisualizerLocals::IsEnabledForSpline(SplineComp))
		{
			return false;
		}
		
		bool bSelectionChanged = false;
		bool bAppendToSelection = InViewportClient->IsShiftPressed();

		const TSet<int32>& SelectedKeys = SelectionState->GetSelectedKeys();
		int32 NumPoints = SplineComp->GetNumberOfSplinePoints();

		// Spline control point selection always uses transparent box selection.
		for (int32 KeyIdx = 0; KeyIdx < NumPoints; KeyIdx++)
		{
			const FVector Pos = SplineComp->GetLocationAtSplinePoint(KeyIdx, ESplineCoordinateSpace::World);

			if (InBox.IsInside(Pos))
			{
				if (!bAppendToSelection || !SelectedKeys.Contains(KeyIdx))
				{
					ChangeSelectionState(KeyIdx, bAppendToSelection);
					bAppendToSelection = true;
					bSelectionChanged = true;
				}
			}
		}

		if (bSelectionChanged)
		{
			SelectionState->ClearSelectedSegmentIndex();
			SelectionState->ClearSelectedTangentHandle();
		}
	}

	return true;
}

bool FSplineComponentVisualizer::HandleFrustumSelect(const FConvexVolume& InFrustum, FEditorViewportClient* InViewportClient, FViewport* InViewport) 
{
	const FScopedTransaction Transaction(LOCTEXT("HandleFrustumSelect", "Frustum Select Spline Points"));

	check(SelectionState);
	SelectionState->Modify();

	ResetTempModes();

	USplineComponent* SplineComp = GetEditedSplineComponent();
	if (SplineComp != nullptr)
	{
		if (!SplineComponentVisualizerLocals::IsEnabledForSpline(SplineComp))
		{
			return false;
		}
		
		bool bSelectionChanged = false;
		bool bAppendToSelection = InViewportClient->IsShiftPressed();

		const TSet<int32>& SelectedKeys = SelectionState->GetSelectedKeys();
		int32 NumPoints = SplineComp->GetNumberOfSplinePoints();

		// Spline control point selection always uses transparent box selection.
		for (int32 KeyIdx = 0; KeyIdx < NumPoints; KeyIdx++)
		{
			const FVector Pos = SplineComp->GetLocationAtSplinePoint(KeyIdx, ESplineCoordinateSpace::World);

			if (InFrustum.IntersectPoint(Pos))
			{
				if (!bAppendToSelection || !SelectedKeys.Contains(KeyIdx))
				{
					ChangeSelectionState(KeyIdx, bAppendToSelection);
					bAppendToSelection = true;
					bSelectionChanged = true;
				}
			}
		}

		if (bSelectionChanged)
		{
			SelectionState->ClearSelectedSegmentIndex();
			SelectionState->ClearSelectedTangentHandle();
		}

		return true;
	}

	return false;
}

bool FSplineComponentVisualizer::HasFocusOnSelectionBoundingBox(FBox& OutBoundingBox)
{
	OutBoundingBox.Init();

	if (USplineComponent* SplineComp = GetEditedSplineComponent())
	{
		if (!SplineComponentVisualizerLocals::IsEnabledForSpline(SplineComp))
		{
			return false;
		}
		
		check(SelectionState);
		const TSet<int32>& SelectedKeys = SelectionState->GetSelectedKeys();
		if (SelectedKeys.Num() > 0)
		{
			// Spline control point selection always uses transparent box selection.
			for (int32 KeyIdx : SelectedKeys)
			{
				check(KeyIdx >= 0); 
				check(KeyIdx < SplineComp->GetNumberOfSplinePoints());

				const FVector Pos = SplineComp->GetLocationAtSplinePoint(KeyIdx, ESplineCoordinateSpace::World);

				OutBoundingBox += Pos;
			}

			OutBoundingBox = OutBoundingBox.ExpandBy(50.f);
			return true;
		}
	}

	return false;
}

bool FSplineComponentVisualizer::HandleSnapTo(const bool bInAlign, const bool bInUseLineTrace, const bool bInUseBounds, const bool bInUsePivot, AActor* InDestination)
{
	ResetTempModes();

	// Does not handle Snap/Align Pivot, Snap/Align Bottom Control Points or Snap/Align to Actor.
	if (bInUsePivot || bInUseBounds || InDestination)
	{
		return false;
	}

	// Note: value of bInUseLineTrace is ignored as we always line trace from control points.

	USplineComponent* SplineComp = GetEditedSplineComponent();

	if (SplineComp != nullptr)
	{
		check(SelectionState);
		const TSet<int32>& SelectedKeys = SelectionState->GetSelectedKeys();
		if (SelectedKeys.Num() > 0)
		{
			int32 LastKeyIndexSelected = SelectionState->GetVerifiedLastKeyIndexSelected(SplineComp->GetNumberOfSplinePoints());
			check(SelectedKeys.Contains(LastKeyIndexSelected));

			SplineComp->Modify();

			int32 NumPoints = SplineComp->GetNumberOfSplinePoints();

			bool bMovedKey = false;

			// Spline control point selection always uses transparent box selection.
			for (int32 KeyIdx : SelectedKeys)
			{
				check(KeyIdx >= 0);
				check(KeyIdx < NumPoints);

				FVector Direction = FVector(0.f, 0.f, -1.f);

				//FInterpCurvePoint<FVector>& EditedPoint = SplinePosition.Points[KeyIdx];
				//FInterpCurvePoint<FQuat>& EditedRotPoint = SplineRotation.Points[KeyIdx];

				FHitResult Hit(1.0f);
				FCollisionQueryParams Params(SCENE_QUERY_STAT(MoveSplineKeyToTrace), true);

				// Find key position in world space
				const FVector CurrentWorldPos = SplineComp->GetLocationAtSplinePoint(KeyIdx, ESplineCoordinateSpace::World);

				if (SplineComp->GetWorld()->LineTraceSingleByChannel(Hit, CurrentWorldPos, CurrentWorldPos + Direction * WORLD_MAX, ECC_WorldStatic, Params))
				{
					SplineComp->SetLocationAtSplinePoint(KeyIdx, Hit.Location, ESplineCoordinateSpace::World, false);

					if (bInAlign)
					{		
						// Get delta rotation between up vector and hit normal
						FVector WorldUpVector = SplineComp->GetUpVectorAtSplineInputKey((float)KeyIdx, ESplineCoordinateSpace::World);
						FQuat DeltaRotate = FQuat::FindBetweenNormals(WorldUpVector, Hit.Normal);

						// Rotate tangent according to delta rotation
						FVector NewTangent = SplineComp->GetLeaveTangentAtSplinePoint(KeyIdx, ESplineCoordinateSpace::World);
						NewTangent = DeltaRotate.RotateVector(NewTangent); // apply world-space delta rotation to world-space tangent
						SplineComp->SetTangentAtSplinePoint(KeyIdx, NewTangent, ESplineCoordinateSpace::World, false);

						// Rotate spline rotation according to delta rotation
						FQuat NewRot = SplineComp->GetRotationAtSplinePoint(KeyIdx, ESplineCoordinateSpace::World).Quaternion();
						NewRot = DeltaRotate * NewRot; // apply world-space rotation
						SplineComp->SetRotationAtSplinePoint(KeyIdx, NewRot.Rotator(), ESplineCoordinateSpace::World, false);
					}

					bMovedKey = true;
				}
			}

			if (bMovedKey)
			{
				SplineComp->UpdateSpline();
				SplineComp->bSplineHasBeenEdited = true;

				NotifyPropertiesModified(SplineComp, SplineProperties);
				if (AActor* Owner = SplineComp->GetOwner())
				{
					Owner->PostEditMove(true);
				}
				
				if (bInAlign)
				{
					SelectionState->Modify();
					SelectionState->SetCachedRotation(SplineComp->GetQuaternionAtSplinePoint(LastKeyIndexSelected, ESplineCoordinateSpace::World));
				}

				GEditor->RedrawLevelEditingViewports(true);
			}

			return true;
		}
	}

	return false;
}

void FSplineComponentVisualizer::TrackingStopped(FEditorViewportClient* InViewportClient, bool bInDidMove)
{
	if (bInDidMove)
	{
		// After dragging, notify that the spline curves property has changed one last time, this time as a EPropertyChangeType::ValueSet :
		USplineComponent* SplineComp = GetEditedSplineComponent();
		NotifyPropertiesModified(SplineComp, SplineProperties, EPropertyChangeType::ValueSet);
		if (AActor* Owner = SplineComp->GetOwner())
		{
			Owner->PostEditMove(true);
		}
	}
}

void FSplineComponentVisualizer::OnSnapKeyToNearestSplinePoint(ESplineComponentSnapMode InSnapMode)
{
	const FScopedTransaction Transaction(LOCTEXT("SnapToNearestSplinePoint", "Snap To Nearest Spline Point"));

	ResetTempModes();

	USplineComponent* SplineComp = GetEditedSplineComponent();
	check(SplineComp != nullptr);
	check(SelectionState);
	const TSet<int32>& SelectedKeys = SelectionState->GetSelectedKeys();
	int32 LastKeyIndexSelected = SelectionState->GetLastKeyIndexSelected();
	check(LastKeyIndexSelected != INDEX_NONE);
	check(LastKeyIndexSelected >= 0);
	check(LastKeyIndexSelected < SplineComp->GetNumberOfSplinePoints());
	check(SelectedKeys.Num() == 1);
	check(SelectedKeys.Contains(LastKeyIndexSelected));

	const FVector WorldPos = SplineComp->GetLocationAtSplinePoint(LastKeyIndexSelected, ESplineCoordinateSpace::World);

	double NearestDistanceSquared = 0.0f;
	USplineComponent* NearestSplineComp = nullptr;
	int32 NearestKeyIndex = INDEX_NONE;

	static const double SnapTol = 5000.0f;
	double SnapTolSquared = SnapTol * SnapTol;

	auto UpdateNearestKey = [WorldPos, SnapTolSquared, &NearestDistanceSquared, &NearestSplineComp, &NearestKeyIndex](USplineComponent* InSplineComp, int InKeyIdx)
	{
		const FVector TestKeyWorldPos = InSplineComp->GetLocationAtSplinePoint(InKeyIdx, ESplineCoordinateSpace::World);
		double TestDistanceSquared = FVector::DistSquared(TestKeyWorldPos, WorldPos);

		if (TestDistanceSquared < SnapTolSquared && (NearestKeyIndex == INDEX_NONE || TestDistanceSquared < NearestDistanceSquared))
		{
			NearestDistanceSquared = TestDistanceSquared;
			NearestSplineComp = InSplineComp;
			NearestKeyIndex = InKeyIdx;
		}
	};

	{
		// Test non-adjacent points on current spline.
		const int32 NumPoints = SplineComp->GetNumberOfSplinePoints();

		// Don't test against current or adjacent points
		TSet<int32> IgnoreIndices;
		IgnoreIndices.Add(LastKeyIndexSelected);
		int32 PrevIndex = LastKeyIndexSelected - 1;
		int32 NextIndex = LastKeyIndexSelected + 1;

		if (PrevIndex >= 0)
		{
			IgnoreIndices.Add(PrevIndex);
		}
		else if (SplineComp->IsClosedLoop())
		{
			IgnoreIndices.Add(NumPoints - 1);
		}

		if (NextIndex < NumPoints)
		{
			IgnoreIndices.Add(NextIndex);
		}
		else if (SplineComp->IsClosedLoop())
		{
			IgnoreIndices.Add(0);
		}

		for (int32 KeyIdx = 0; KeyIdx < NumPoints; KeyIdx++)
		{
			if (!IgnoreIndices.Contains(KeyIdx))
			{
				UpdateNearestKey(SplineComp, KeyIdx);
			}
		}
	}

	// Test whether component and its owning actor are valid and visible
	auto IsValidAndVisible = [](const USplineComponent* Comp)
	{
		return (Comp && !Comp->IsBeingDestroyed() && Comp->IsVisibleInEditor() &&
				Comp->GetOwner() && IsValid(Comp->GetOwner()) && !Comp->GetOwner()->IsHiddenEd());
	};

	// Next search all spline components for nearest point on splines, excluding current spline
	// Only test points in splines whose bounding box contains this point.
	for (TObjectIterator<USplineComponent> SplineIt; SplineIt; ++SplineIt)
	{
		USplineComponent* TestComponent = *SplineIt;

		// Ignore current spline and those which are not valid 
		if (TestComponent && TestComponent != SplineComp && IsValidAndVisible(TestComponent) &&
			!FMath::IsNearlyZero(TestComponent->Bounds.SphereRadius))
		{
			FBox TestComponentBoundingBox = TestComponent->Bounds.GetBox().ExpandBy(FVector(SnapTol, SnapTol, SnapTol));

			if (TestComponentBoundingBox.IsInsideOrOn(WorldPos))
			{
				const int32 NumPoints = SplineComp->GetNumberOfSplinePoints();
				for (int32 KeyIdx = 0; KeyIdx < NumPoints; KeyIdx++)
				{
					UpdateNearestKey(TestComponent, KeyIdx);
				}
			}
		}
	}

	if (!NearestSplineComp || NearestKeyIndex == INDEX_NONE)
	{
		UE_LOGF(LogSplineComponentVisualizer, Warning, "No nearest spline point found.");
		return;
	}

	// Copy position
	const FVector NearestWorldPos = SplineComp->GetLocationAtSplinePoint(NearestKeyIndex, ESplineCoordinateSpace::World);
	FVector NearestWorldUpVector(0.0f, 0.0f, 1.0f);
	FVector NearestWorldTangent(0.0f, 1.0f, 0.0f);
	FVector NearestWorldScale(1.0f, 1.0f, 1.0f);
	USplineMetadata* NearestSplineMetadata = nullptr;

	if (InSnapMode == ESplineComponentSnapMode::AlignToTangent || InSnapMode == ESplineComponentSnapMode::AlignPerpendicularToTangent)
	{
		// Get tangent
		NearestWorldTangent = SplineComp->GetArriveTangentAtSplinePoint(NearestKeyIndex, ESplineCoordinateSpace::World); // convert local-space tangent vectors to world-space

		// Get up vector
		NearestWorldUpVector = NearestSplineComp->GetUpVectorAtSplinePoint(NearestKeyIndex, ESplineCoordinateSpace::World);

		// Get scale, only when aligning parallel
		if (InSnapMode == ESplineComponentSnapMode::AlignToTangent)
		{
			const FVector NearestScale = SplineComp->GetScaleAtSplinePoint(NearestKeyIndex);
			NearestWorldScale = SplineComp->GetComponentTransform().GetScale3D() * NearestScale; // convert local-space rotation to world-space
		}

		// Get metadata (only when aligning)
		USplineMetadata* SplineMetadata = SplineComp->GetSplinePointsMetadata();
		NearestSplineMetadata = SplineMetadata ? NearestSplineComp->GetSplinePointsMetadata() : nullptr;
	}

	SnapKeyToTransform(InSnapMode, NearestWorldPos, NearestWorldUpVector, NearestWorldTangent, NearestWorldScale, NearestSplineMetadata, NearestKeyIndex);
}

void FSplineComponentVisualizer::OnSnapKeyToActor(const ESplineComponentSnapMode InSnapMode)
{
	ResetTempModes();
	SetSnapToActorMode(true, InSnapMode);
}

void FSplineComponentVisualizer::SnapKeyToActor(const AActor* InActor, const ESplineComponentSnapMode InSnapMode)
{
	const FScopedTransaction Transaction(LOCTEXT("SnapToActor", "Snap To Actor"));

	if (InActor && IsSingleKeySelected())
	{
		const FVector ActorLocation = InActor->GetActorLocation();
		const FVector ActorUpVector = InActor->GetActorUpVector();
		const FVector ActorForwardVector = InActor->GetActorForwardVector();
		const FVector UniformScale(1.0f, 1.0f, 1.0f);

		SnapKeyToTransform(InSnapMode, ActorLocation, ActorUpVector, ActorForwardVector, UniformScale);
	}
}

void FSplineComponentVisualizer::SnapKeyToTransform(const ESplineComponentSnapMode InSnapMode,
	const FVector& InWorldPos,
	const FVector& InWorldUpVector,
	const FVector& InWorldForwardVector,
	const FVector& InScale,
	const USplineMetadata* InCopySplineMetadata,
	const int32 InCopySplineMetadataKey)
{
	USplineComponent* SplineComp = GetEditedSplineComponent();
	check(SplineComp != nullptr);
	check(SelectionState);
	const TSet<int32>& SelectedKeys = SelectionState->GetSelectedKeys();
	int32 LastKeyIndexSelected = SelectionState->GetVerifiedLastKeyIndexSelected(SplineComp->GetNumberOfSplinePoints());
	check(SelectedKeys.Num() == 1);
	check(SelectedKeys.Contains(LastKeyIndexSelected));

	SplineComp->Modify();
	if (AActor* Owner = SplineComp->GetOwner())
	{
		Owner->Modify();
	}

	// Copy position
	SplineComp->SetLocationAtSplinePoint(LastKeyIndexSelected, InWorldPos, ESplineCoordinateSpace::World, false);

	if (InSnapMode == ESplineComponentSnapMode::AlignToTangent || InSnapMode == ESplineComponentSnapMode::AlignPerpendicularToTangent)
	{
		// Copy tangents
		const FVector WorldUpVector = InWorldUpVector.GetSafeNormal();
		const FVector WorldForwardVector = InWorldForwardVector.GetSafeNormal();

		// Copy tangents
		FVector NewTangent = WorldForwardVector;

		if (InSnapMode == ESplineComponentSnapMode::AlignPerpendicularToTangent)
		{
			// Rotate tangent by 90 degrees
			const FQuat DeltaRotate(WorldUpVector, UE_HALF_PI);
			NewTangent = DeltaRotate.RotateVector(NewTangent);
		}

		const FVector Tangent = SplineComp->GetArriveTangentAtSplinePoint(LastKeyIndexSelected, ESplineCoordinateSpace::World);

		// Swap the tangents if they are not pointing in the same general direction
		double CurrentAngle = FMath::Acos(FVector::DotProduct(Tangent, NewTangent) / Tangent.Size());
		if (CurrentAngle > UE_HALF_PI)
		{
			NewTangent = SplineComp->GetComponentTransform().GetRotation().Inverse().RotateVector(NewTangent * -1.0f) * Tangent.Size(); // convert world-space tangent vectors into local-space
		}
		else
		{
			NewTangent = SplineComp->GetComponentTransform().GetRotation().Inverse().RotateVector(NewTangent) * Tangent.Size(); // convert world-space tangent vectors into local-space
		}

		// Update tangent
		SplineComp->SetTangentAtSplinePoint(LastKeyIndexSelected, NewTangent, ESplineCoordinateSpace::Local, false);

		// Copy rotation, it is only used to determine up vector so no need to adjust it 
		const FQuat Rot = FQuat::FindBetweenNormals(FVector(0.0f, 0.0f, 1.0f), WorldUpVector);
		SplineComp->SetRotationAtSplinePoint(LastKeyIndexSelected, Rot.Rotator(), ESplineCoordinateSpace::World, false);

		// Copy scale, only when aligning parallel
		if (InSnapMode == ESplineComponentSnapMode::AlignToTangent)
		{
			const FVector SplineCompScale = SplineComp->GetComponentTransform().GetScale3D();
			FVector NewScale;
			NewScale.X = FMath::IsNearlyZero(SplineCompScale.X) ? InScale.X : InScale.X / SplineCompScale.X;
			NewScale.Y = FMath::IsNearlyZero(SplineCompScale.Y) ? InScale.Y : InScale.Y / SplineCompScale.Y;
			NewScale.Z = FMath::IsNearlyZero(SplineCompScale.Z) ? InScale.Z : InScale.Z / SplineCompScale.Z;
			SplineComp->SetScaleAtSplinePoint(LastKeyIndexSelected, NewScale, false);
		}

	}

	// Copy metadata
	if (InCopySplineMetadata)
	{
		if (USplineMetadata* SplineMetadata = SplineComp->GetSplinePointsMetadata())
		{
			SplineMetadata->CopyPoint(InCopySplineMetadata, InCopySplineMetadataKey, LastKeyIndexSelected);
		}
	}

	SplineComp->UpdateSpline();
	SplineComp->bSplineHasBeenEdited = true;

	NotifyPropertiesModified(SplineComp, SplineProperties);
	if (AActor* Owner = SplineComp->GetOwner())
	{
		Owner->PostEditMove(true);
	}

	if (InSnapMode == ESplineComponentSnapMode::AlignToTangent || InSnapMode == ESplineComponentSnapMode::AlignPerpendicularToTangent)
	{
		SelectionState->Modify();
		SelectionState->SetCachedRotation(SplineComp->GetQuaternionAtSplinePoint(LastKeyIndexSelected, ESplineCoordinateSpace::World));
	}

	GEditor->RedrawLevelEditingViewports(true);
}

void FSplineComponentVisualizer::OnSnapAllToAxis(EAxis::Type InAxis)
{
	const FScopedTransaction Transaction(LOCTEXT("SnapAllToSelectedAxis", "Snap All To Selected Axis"));

	ResetTempModes();

	USplineComponent* SplineComp = GetEditedSplineComponent();
	check(SplineComp != nullptr);
	check(SelectionState);
	const TSet<int32>& SelectedKeys = SelectionState->GetSelectedKeys();
	int32 LastKeyIndexSelected = SelectionState->GetVerifiedLastKeyIndexSelected(SplineComp->GetNumberOfSplinePoints());
	check(SelectedKeys.Num() == 1);
	check(SelectedKeys.Contains(LastKeyIndexSelected));
	check(InAxis == EAxis::X || InAxis == EAxis::Y || InAxis == EAxis::Z);

	TArray<int32> SnapKeys;
	for (int32 KeyIdx = 0; KeyIdx < SplineComp->GetNumberOfSplinePoints(); KeyIdx++)
	{
		if (KeyIdx != LastKeyIndexSelected)
		{
			SnapKeys.Add(KeyIdx);
		}
	}

	SnapKeysToLastSelectedAxisPosition(InAxis, SnapKeys);
}

void FSplineComponentVisualizer::OnSnapSelectedToAxis(EAxis::Type InAxis)
{
	const FScopedTransaction Transaction(LOCTEXT("SnapSelectedToLastAxis", "Snap Selected To Axis"));

	ResetTempModes();

	USplineComponent* SplineComp = GetEditedSplineComponent();
	check(SelectionState);
	const TSet<int32>& SelectedKeys = SelectionState->GetSelectedKeys();
	int32 LastKeyIndexSelected = SelectionState->GetVerifiedLastKeyIndexSelected(SplineComp->GetNumberOfSplinePoints());
	check(SplineComp != nullptr);	
	check(SelectedKeys.Num() > 1);

	TArray<int32> SnapKeys;
	for (int32 KeyIdx : SelectedKeys)
	{
		if (KeyIdx != LastKeyIndexSelected)
		{
			SnapKeys.Add(KeyIdx);
		}
	}

	SnapKeysToLastSelectedAxisPosition(InAxis, SnapKeys);
}

void FSplineComponentVisualizer::OnStraightenKey(int32 Direction)
{
	const FScopedTransaction Transaction(LOCTEXT("Straighten To Previous", "Straighten Points Toward Previous"));

	USplineComponent* SplineComp = GetEditedSplineComponent();
	check(SplineComp != nullptr);
	check(SelectionState);
	const TSet<int32>& SelectedKeys = SelectionState->GetSelectedKeys();
	int32 LastKeyIndexSelected = SelectionState->GetVerifiedLastKeyIndexSelected(SplineComp->GetNumberOfSplinePoints());

	for (int32 CurrentKey : SelectedKeys)
	{
		int32 ToKey = CurrentKey + Direction;
		if (ToKey != INDEX_NONE && ToKey < SplineComp->GetNumberOfSplinePoints())
		{
			StraightenKey(CurrentKey, ToKey);
		}
	}

	SplineComp->UpdateSpline();
	SplineComp->bSplineHasBeenEdited = true;

	NotifyPropertiesModified(SplineComp, SplineProperties);
	if (AActor* OwnerActor = SplineComp->GetOwner())
	{
		OwnerActor->PostEditMove(true);
	}

	SelectionState->Modify();
	SelectionState->SetCachedRotation(SplineComp->GetQuaternionAtSplinePoint(LastKeyIndexSelected, ESplineCoordinateSpace::World));

	GEditor->RedrawLevelEditingViewports(true);
}

void FSplineComponentVisualizer::StraightenKey(int32 KeyToStraighten, int32 KeyToStraightenToward)
{
	USplineComponent* SplineComp = GetEditedSplineComponent();
	check(SplineComp != nullptr);

	const FVector::FReal TangentLength = SplineComp->GetTangentAtSplinePoint(KeyToStraighten, ESplineCoordinateSpace::Local).Length();
	FVector StraightenLocation = SplineComp->GetLocationAtSplinePoint(KeyToStraighten, ESplineCoordinateSpace::Local);
	FVector TowardLocation = SplineComp->GetLocationAtSplinePoint(KeyToStraightenToward, ESplineCoordinateSpace::Local);
	FVector Direction = TowardLocation - StraightenLocation;
	Direction.Normalize();

	FVector NewTangent = Direction * TangentLength * (KeyToStraighten > KeyToStraightenToward ? 1 : -1);
	SplineComp->SetTangentAtSplinePoint(KeyToStraighten, -NewTangent, ESplineCoordinateSpace::Local);
}

void FSplineComponentVisualizer::OnToggleSnapTangentAdjustment()
{
	USplineComponent* SplineComp = GetEditedSplineComponent();
	check(SplineComp != nullptr);

	SplineComp->Modify();
	if (AActor* Owner = SplineComp->GetOwner())
	{
		Owner->Modify();
	}

	SplineComp->bAdjustTangentsOnSnap = !SplineComp->bAdjustTangentsOnSnap;

	TArray<FProperty*> Properties = SplineProperties;
	Properties.Add(FindFProperty<FProperty>(USplineComponent::StaticClass(), GET_MEMBER_NAME_CHECKED(USplineComponent, bAdjustTangentsOnSnap)));
	NotifyPropertiesModified(SplineComp, Properties);

	GEditor->RedrawLevelEditingViewports(true);
}

bool FSplineComponentVisualizer::IsSnapTangentAdjustment() const
{
	USplineComponent* SplineComp = GetEditedSplineComponent();
	return SplineComp ? SplineComp->bAdjustTangentsOnSnap : false;
}

void FSplineComponentVisualizer::SnapKeysToLastSelectedAxisPosition(const EAxis::Type InAxis, TArray<int32> InSnapKeys)
{
	USplineComponent* SplineComp = GetEditedSplineComponent();
	check(SplineComp != nullptr);
	check(SelectionState);
	check(InAxis == EAxis::X || InAxis == EAxis::Y || InAxis == EAxis::Z);
	int32 LastKeyIndexSelected = SelectionState->GetVerifiedLastKeyIndexSelected(SplineComp->GetNumberOfSplinePoints());

	SplineComp->Modify();
	if (AActor* Owner = SplineComp->GetOwner())
	{
		Owner->Modify();
	}

	const FVector WorldPos = SplineComp->GetLocationAtSplinePoint(LastKeyIndexSelected, ESplineCoordinateSpace::World);

	for (int32 KeyIdx : InSnapKeys)
	{
		if (KeyIdx >= 0 && KeyIdx < SplineComp->GetNumberOfSplinePoints())
		{
			// Copy position
			FVector NewWorldPos = SplineComp->GetLocationAtSplinePoint(KeyIdx, ESplineCoordinateSpace::World);
			if (InAxis == EAxis::X)
			{
				NewWorldPos.X = WorldPos.X;
			}
			else if (InAxis == EAxis::Y)
			{
				NewWorldPos.Y = WorldPos.Y;
			}
			else
			{
				NewWorldPos.Z = WorldPos.Z;
			}

			SplineComp->SetLocationAtSplinePoint(KeyIdx, NewWorldPos, ESplineCoordinateSpace::World);

			// Set point to auto so its tangents will be auto-adjusted after snapping
			if (SplineComp->bAdjustTangentsOnSnap)
			{
				SplineComp->SetSplinePointType(KeyIdx, ConvertInterpCurveModeToSplinePointType(CIM_CurveAuto));
			}
		}
	}

	SplineComp->UpdateSpline();
	SplineComp->bSplineHasBeenEdited = true;

	NotifyPropertiesModified(SplineComp, SplineProperties);
	if (AActor* Owner = SplineComp->GetOwner())
	{
		Owner->PostEditMove(true);
	}

	SelectionState->Modify();
	SelectionState->SetCachedRotation(SplineComp->GetQuaternionAtSplinePoint(LastKeyIndexSelected, ESplineCoordinateSpace::World));

	GEditor->RedrawLevelEditingViewports(true);
}

void FSplineComponentVisualizer::EndEditing()
{
	// Ignore if there is an undo/redo operation in progress
	if (!GIsTransacting)
	{
		check(SelectionState);
		SelectionState->Modify();

		if (USplineComponent* SplineComp = GetEditedSplineComponent())
		{
			ChangeSelectionState(INDEX_NONE, false);
			SelectionState->ClearSelectedSegmentIndex();
			SelectionState->ClearSelectedTangentHandle();
		}
		SelectionState->SetSplinePropertyPath(FComponentPropertyPath());

		ResetTempModes();
	}
}

void FSplineComponentVisualizer::ResetTempModes()
{
	SetSnapToActorMode(false);
}

void FSplineComponentVisualizer::SetSnapToActorMode(const bool bInIsSnappingToActor, const ESplineComponentSnapMode InSnapMode)
{
	bIsSnappingToActor = bInIsSnappingToActor;
	SnapToActorMode = InSnapMode;
}

bool FSplineComponentVisualizer::GetSnapToActorMode(ESplineComponentSnapMode& OutSnapMode) const 
{
	OutSnapMode = SnapToActorMode;
	return bIsSnappingToActor;
}

void FSplineComponentVisualizer::OnDuplicateKey()
{
	const FScopedTransaction Transaction(LOCTEXT("DuplicateSplinePoint", "Duplicate Spline Point"));

	ResetTempModes();
	
	USplineComponent* SplineComp = GetEditedSplineComponent();
	check(SplineComp != nullptr);
	check(SelectionState);
	const TSet<int32>& SelectedKeys = SelectionState->GetSelectedKeys();
	int32 LastKeyIndexSelected = SelectionState->GetVerifiedLastKeyIndexSelected(SplineComp->GetNumberOfSplinePoints());
	check(SelectionState->GetSelectedKeys().Num() > 0);
	check(SelectionState->GetSelectedKeys().Contains(LastKeyIndexSelected));

	SplineComp->Modify();
	if (AActor* Owner = SplineComp->GetOwner())
	{
		Owner->Modify();
	}

	// Get a sorted list of all the selected indices, highest to lowest
	TArray<int32> SelectedKeysSorted;
	for (int32 SelectedKeyIndex : SelectedKeys)
	{
		SelectedKeysSorted.Add(SelectedKeyIndex);
	}
	SelectedKeysSorted.Sort([](int32 A, int32 B) { return A > B; });

	// Insert duplicates into the list, highest index first, so that the lower indices remain the same
	for (int32 SelectedKeyIndex : SelectedKeysSorted)
	{
		check(SelectedKeyIndex >= 0);
		check(SelectedKeyIndex < SplineComp->GetNumberOfSplinePoints());

		const int32 InsertIndex = SelectedKeyIndex + 1;
		
		FSplinePoint NewPoint = SplineComp->GetSplinePointAt(SelectedKeyIndex, ESplineCoordinateSpace::Local);
		SplineComp->AddSplinePointAtIndex(NewPoint.Position, InsertIndex, ESplineCoordinateSpace::Local, false);
		SplineComp->SetScaleAtSplinePoint(InsertIndex, NewPoint.Scale, false);
		SplineComp->SetQuaternionAtSplinePoint(InsertIndex, NewPoint.Rotation.Quaternion(), ESplineCoordinateSpace::Local, false);
		SplineComp->SetTangentsAtSplinePoint(InsertIndex, NewPoint.ArriveTangent, NewPoint.LeaveTangent, ESplineCoordinateSpace::Local, false);
		SplineComp->SetSplinePointType(InsertIndex, NewPoint.Type, false);

		// USplineComponent::AddSplinePointAtIndex inserts a default metadata point, but the expectation is that the metadata point is duplicated.
		// This can be implemented by overwriting the inserted point with the metadata point at the selected index.
		if (USplineMetadata* Metadata = SplineComp->GetSplinePointsMetadata())
		{
			Metadata->CopyPoint(Metadata, SelectedKeyIndex, InsertIndex);
		}
	}

	SelectionState->Modify();

	// Repopulate the selected keys
	TSet<int32>& NewSelectedKeys = SelectionState->ModifySelectedKeys();
	NewSelectedKeys.Empty();
	int32 Offset = SelectedKeysSorted.Num();
	for (int32 SelectedKeyIndex : SelectedKeysSorted)
	{
		NewSelectedKeys.Add(SelectedKeyIndex + Offset);

		if (SelectionState->GetLastKeyIndexSelected() == SelectedKeyIndex)
		{
			SelectionState->SetLastKeyIndexSelected(SelectionState->GetLastKeyIndexSelected() + Offset);
		}

		Offset--;
	}

	// Unset tangent handle selection
	SelectionState->ClearSelectedTangentHandle();

	SplineComp->UpdateSpline();
	SplineComp->bSplineHasBeenEdited = true;

	NotifyPropertiesModified(SplineComp, SplineProperties);
	if (AActor* Owner = SplineComp->GetOwner())
	{
		Owner->PostEditMove(true);
	}

	if (NewSelectedKeys.Num() == 1)
	{
		SelectionState->SetCachedRotation(SplineComp->GetQuaternionAtSplinePoint(SelectionState->GetLastKeyIndexSelected(), ESplineCoordinateSpace::World));
	}

	GEditor->RedrawLevelEditingViewports(true);
}

bool FSplineComponentVisualizer::CanAddKeyToSegment() const
{
	USplineComponent* SplineComp = GetEditedSplineComponent();
	if (SplineComp == nullptr)
	{
		return false;
	}

	check(SelectionState);
	int32 SelectedSegmentIndex = SelectionState->GetSelectedSegmentIndex();
	return (SelectedSegmentIndex != INDEX_NONE && SelectedSegmentIndex >=0 && SelectedSegmentIndex < SplineComp->GetNumberOfSplineSegments());
}

void FSplineComponentVisualizer::OnAddKeyToSegment()
{
	const FScopedTransaction Transaction(LOCTEXT("AddSplinePoint", "Add Spline Point"));

	ResetTempModes();

	USplineComponent* SplineComp = GetEditedSplineComponent();
	check(SplineComp != nullptr);
	check(SelectionState);
	check(SelectionState->GetSelectedTangentHandle() == INDEX_NONE);
	check(SelectionState->GetSelectedTangentHandleType() == ESelectedTangentHandle::None);

	SelectionState->Modify();
	
	SplineComp->UpdateSpline();
	
	SplitSegment(SelectionState->GetSelectedSplinePosition(), SelectionState->GetSelectedSegmentIndex());

	SelectionState->SetSelectedSegmentIndex(INDEX_NONE);
	SelectionState->SetSelectedSplinePosition(FVector::ZeroVector);
	SelectionState->SetCachedRotation(SplineComp->GetQuaternionAtSplinePoint(SelectionState->GetLastKeyIndexSelected(), ESplineCoordinateSpace::World));
}

bool FSplineComponentVisualizer::DuplicateKeyForAltDrag(const FVector& InDrag)
{
	USplineComponent* SplineComp = GetEditedSplineComponent();
	check(SplineComp != nullptr);
	check(SelectionState);
	const int32 NumPoints = SplineComp->GetNumberOfSplinePoints();
	const TSet<int32>& SelectedKeys = SelectionState->GetSelectedKeys();
	int32 LastKeyIndexSelected = SelectionState->GetVerifiedLastKeyIndexSelected(NumPoints);
	check(SelectedKeys.Num() == 1);
	check(SelectedKeys.Contains(LastKeyIndexSelected));

	SplineComp->UpdateSpline();
	
	// When dragging from end point, maximum angle is 60 degrees from attached segment
	// to determine whether to split existing segment or create a new point
	static const double Angle60 = 1.0472;

	// Insert duplicates into the list, highest index first, so that the lower indices remain the same
	//FInterpCurveVector& SplinePosition = SplineComp->GetSplinePointsPosition();

	// Find key position in world space
	int32 CurrentIndex = LastKeyIndexSelected;
	const FVector CurrentKeyWorldPos = SplineComp->GetLocationAtSplinePoint(CurrentIndex, ESplineCoordinateSpace::World);

	// Determine direction to insert new point				
	bool bHasPrevKey = SplineComp->IsClosedLoop() || CurrentIndex > 0;
	double PrevAngle = 0.0f;
	if (bHasPrevKey)
	{
		// Wrap index around for closed-looped splines
		int32 PrevKeyIndex = (CurrentIndex > 0 ? CurrentIndex - 1 : NumPoints - 1);
		FVector PrevKeyWorldPos = SplineComp->GetLocationAtSplinePoint(PrevKeyIndex, ESplineCoordinateSpace::World);
		FVector SegmentDirection = PrevKeyWorldPos - CurrentKeyWorldPos;
		if (!SegmentDirection.IsZero())
		{
			PrevAngle = FMath::Acos(FVector::DotProduct(InDrag, SegmentDirection) / (InDrag.Size() * SegmentDirection.Size()));
		}
		else
		{
			PrevAngle = Angle60;
		}
	}

	bool bHasNextKey = SplineComp->IsClosedLoop() || CurrentIndex + 1 < NumPoints;
	double NextAngle = 0.0f;
	if (bHasNextKey)
	{
		// Wrap index around for closed-looped splines
		int32 NextKeyIndex = (CurrentIndex + 1 < NumPoints ? CurrentIndex + 1 : 0);
		FVector NextKeyWorldPos = SplineComp->GetLocationAtSplinePoint(NextKeyIndex, ESplineCoordinateSpace::World);
		FVector SegmentDirection = NextKeyWorldPos - CurrentKeyWorldPos;
		if (!SegmentDirection.IsZero())
		{
			NextAngle = FMath::Acos(FVector::DotProduct(InDrag, SegmentDirection) / (InDrag.Size() * SegmentDirection.Size()));
		}
		else
		{
			NextAngle = Angle60;
		}
	}

	// Set key index to which the drag will be applied after duplication
	int32 SegmentIndex = CurrentIndex;

	if ((bHasPrevKey && bHasNextKey && PrevAngle < NextAngle) ||
		(bHasPrevKey && !bHasNextKey && PrevAngle < Angle60) ||
		(!bHasPrevKey && bHasNextKey && NextAngle >= Angle60))
	{
		SegmentIndex--;
	}

	// Wrap index around for closed-looped splines
	const int32 NumSegments = SplineComp->GetNumberOfSplineSegments();
	if (SplineComp->IsClosedLoop() && SegmentIndex < 0)
	{
		SegmentIndex = NumSegments - 1;
	}

	FVector WorldPos = CurrentKeyWorldPos + InDrag;

	// Split existing segment or add new segment
	if (SegmentIndex >= 0 && SegmentIndex < NumSegments)
	{
		bool bCopyFromSegmentBeginIndex = (LastKeyIndexSelected == SegmentIndex);
		SplitSegment(WorldPos, SegmentIndex, bCopyFromSegmentBeginIndex);

	}
	else
	{
		AddSegment(WorldPos, (SegmentIndex > 0));
		bUpdatingAddSegment = true;
	}

	// Unset tangent handle selection
	SelectionState->Modify();
	SelectionState->ClearSelectedTangentHandle();
	
	return true;
}

bool FSplineComponentVisualizer::UpdateDuplicateKeyForAltDrag(const FVector& InDrag)
{
	if (bUpdatingAddSegment)
	{
		UpdateAddSegment(InDrag);
	}
	else
	{
		UpdateSplitSegment(InDrag);
	}

	return true;
}

float FSplineComponentVisualizer::FindNearest(const FVector& InLocalPos, int32 InSegmentIndex, FVector& OutSplinePos, FVector& OutSplineTangent) const
{
	USplineComponent* SplineComp = GetEditedSplineComponent();
	check(SplineComp != nullptr);
	check(InSegmentIndex != INDEX_NONE);
	check(InSegmentIndex >= 0);
	check(InSegmentIndex < SplineComp->GetNumberOfSplineSegments());

	FVector WorldPos = SplineComp->GetComponentTransform().InverseTransformPosition(InLocalPos);
	float t = SplineComp->FindInputKeyOnSegmentClosestToWorldLocation(WorldPos, InSegmentIndex);
	OutSplinePos = SplineComp->GetLocationAtSplineInputKey(t, ESplineCoordinateSpace::Local);
	OutSplineTangent = SplineComp->GetTangentAtSplineInputKey(t, ESplineCoordinateSpace::Local);

	return t;
}

void FSplineComponentVisualizer::SplitSegment(const FVector& InWorldPos, int32 InSegmentIndex, bool bCopyFromSegmentBeginIndex /* = true */)
{
	USplineComponent* SplineComp = GetEditedSplineComponent();
	check(SplineComp != nullptr);
	check(SelectionState);
	const TSet<int32>& SelectedKeys = SelectionState->GetSelectedKeys();
	int32 LastKeyIndexSelected = SelectionState->GetVerifiedLastKeyIndexSelected(SplineComp->GetNumberOfSplinePoints());
	check(SelectedKeys.Num() == 1);
	check(SelectedKeys.Contains(LastKeyIndexSelected));
	check(InSegmentIndex != INDEX_NONE);
	check(InSegmentIndex >= 0);
	check(InSegmentIndex < SplineComp->GetNumberOfSplineSegments());

	SplineComp->Modify();
	if (AActor* Owner = SplineComp->GetOwner())
	{
		Owner->Modify();
	}

	// Compute local pos
	FVector LocalPos = SplineComp->GetComponentTransform().InverseTransformPosition(InWorldPos);

	FVector SplinePos, SplineTangent;
	float SplineParam = FindNearest(LocalPos, InSegmentIndex, SplinePos, SplineTangent);
	float t = SplineParam - static_cast<float>(InSegmentIndex);

	if (bDuplicatingSplineKey)
	{
		DuplicateCacheSplitSegmentParam = t;
	}

	int32 SegmentBeginIndex = InSegmentIndex;
	int32 SegmentSplitIndex = InSegmentIndex + 1;
	int32 SegmentEndIndex = SegmentSplitIndex;
	if (SplineComp->IsClosedLoop() && SegmentEndIndex >= SplineComp->GetNumberOfSplinePoints())
	{
		SegmentEndIndex = 0;
	}

	// Compute interpolated scale
	FVector NewScale;
	FVector PrevScale = SplineComp->GetScaleAtSplinePoint(SegmentBeginIndex);
	FVector NextScale = SplineComp->GetScaleAtSplinePoint(SegmentEndIndex);
	NewScale = FMath::LerpStable(PrevScale, NextScale, t);

	// Compute interpolated rot
	FQuat NewRot;
	FQuat PrevRot = SplineComp->GetRotationAtSplinePoint(SegmentBeginIndex, ESplineCoordinateSpace::Local).Quaternion();
	FQuat NextRot = SplineComp->GetRotationAtSplinePoint(SegmentEndIndex, ESplineCoordinateSpace::Local).Quaternion();
	NewRot = FMath::Lerp(PrevRot, NextRot, t);

	// Determine which index to use when copying interp mode
	int32 SourceIndex = bCopyFromSegmentBeginIndex ? SegmentBeginIndex : SegmentEndIndex;
	ESplinePointType::Type SourceSplinePointType = SplineComp->GetSplinePointType(SourceIndex);
	// If the spline interpolation mode of the source point is a custom tangent curve, change it to be an auto curve
	ESplinePointType::Type NewSplinePointType = SourceSplinePointType == ESplinePointType::CurveCustomTangent ? ESplinePointType::Curve : SourceSplinePointType;
	SplineComp->AddSplinePointAtIndex(LocalPos, SegmentSplitIndex, ESplineCoordinateSpace::Local, false);
	SplineComp->SetQuaternionAtSplinePoint(SegmentSplitIndex, NewRot, ESplineCoordinateSpace::Local, false);
	SplineComp->SetScaleAtSplinePoint(SegmentSplitIndex, NewScale, false);
	SplineComp->SetSplinePointType(SegmentSplitIndex, NewSplinePointType, false);

	// Update metadata
	if (USplineMetadata* SplineMetadata = SplineComp->GetSplinePointsMetadata())
	{
		SplineMetadata->UpdatePoint(SegmentSplitIndex, t, SplineComp->IsClosedLoop());
	}
	
	// Set selection to new key
	ChangeSelectionState(SegmentSplitIndex, false);

	SplineComp->UpdateSpline();
	SplineComp->bSplineHasBeenEdited = true;

	NotifyPropertiesModified(SplineComp, SplineProperties);
	if (AActor* Owner = SplineComp->GetOwner())
	{
		Owner->PostEditMove(true);
	}


	GEditor->RedrawLevelEditingViewports(true);
}

void FSplineComponentVisualizer::UpdateSplitSegment(const FVector& InDrag)
{
	const FScopedTransaction Transaction(LOCTEXT("UpdateSplitSegment", "Update Split Segment"));

	USplineComponent* SplineComp = GetEditedSplineComponent();
	check(SplineComp != nullptr);
	check(SelectionState);
	const TSet<int32>& SelectedKeys = SelectionState->GetSelectedKeys();
	int32 LastKeyIndexSelected = SelectionState->GetLastKeyIndexSelected();
	check(LastKeyIndexSelected != INDEX_NONE);
	check(SelectedKeys.Num() == 1);
	check(SelectedKeys.Contains(LastKeyIndexSelected));
	// LastKeyIndexSelected is the newly created point when splitting a segment with alt-drag. 
	// Check that it is an internal point, not an end point.
	check(LastKeyIndexSelected > 0);
	check(LastKeyIndexSelected < SplineComp->GetNumberOfSplineSegments());

	SplineComp->Modify();
	if (AActor* Owner = SplineComp->GetOwner())
	{
		Owner->Modify();
	}

	int32 SegmentStartIndex = LastKeyIndexSelected - 1;
	int32 SegmentSplitIndex = LastKeyIndexSelected;
	int32 SegmentEndIndex = LastKeyIndexSelected + 1;

	// Wrap end point if on last segment of closed-looped spline
	if (SplineComp->IsClosedLoop() && SegmentEndIndex >= SplineComp->GetNumberOfSplineSegments())
	{
		SegmentEndIndex = 0;
	}

	// Find key position in world space
	const FVector CurrentWorldPos = SplineComp->GetLocationAtSplinePoint(SegmentSplitIndex, ESplineCoordinateSpace::World);

	// Move in world space
	const FVector NewWorldPos = CurrentWorldPos + InDrag;

	// Convert back to local space
	FVector LocalPos = SplineComp->GetComponentTransform().InverseTransformPosition(NewWorldPos);

	FVector SplinePos0, SplinePos1;
	FVector SplineTangent0, SplineTangent1;
	float t = 0.0f;
	float SplineParam0 = FindNearest(LocalPos, SegmentStartIndex, SplinePos0, SplineTangent0);
	float t0 = SplineParam0 - static_cast<float>(SegmentStartIndex);
	float SplineParam1 = FindNearest(LocalPos, SegmentSplitIndex, SplinePos1, SplineTangent1);
	float t1 = SplineParam1 - static_cast<float>(SegmentSplitIndex);

	// Calculate params
	if (FVector::Distance(LocalPos, SplinePos0) < FVector::Distance(LocalPos, SplinePos1))
	{
		t = DuplicateCacheSplitSegmentParam * t0;
	}
	else
	{
		t = DuplicateCacheSplitSegmentParam + (1 - DuplicateCacheSplitSegmentParam) * t1;
	}
	DuplicateCacheSplitSegmentParam = t;

	// Update location
	SplineComp->SetLocationAtSplinePoint(SegmentSplitIndex, LocalPos, ESplineCoordinateSpace::Local, false);

	// Update scale
	const FVector PrevScale = SplineComp->GetScaleAtSplinePoint(SegmentStartIndex);
	const FVector NextScale = SplineComp->GetScaleAtSplinePoint(SegmentEndIndex);
	SplineComp->SetScaleAtSplinePoint(SegmentSplitIndex, FMath::LerpStable(PrevScale, NextScale, t), false);

	// Update rot
	ESplinePointType::Type SplinePointType = SplineComp->GetSplinePointType(SegmentSplitIndex);
	FQuat PrevRot = SplineComp->GetRotationAtSplinePoint(SegmentStartIndex, ESplineCoordinateSpace::Local).Quaternion();
	FQuat NextRot = SplineComp->GetRotationAtSplinePoint(SegmentEndIndex, ESplineCoordinateSpace::Local).Quaternion();
	SplineComp->SetRotationAtSplinePoint(SegmentSplitIndex, FMath::Lerp(PrevRot, NextRot, t).Rotator(), ESplineCoordinateSpace::Local, false);
	SplineComp->SetSplinePointType(SegmentSplitIndex, SplinePointType, false);

	// Update metadata
	if (USplineMetadata* SplineMetadata = SplineComp->GetSplinePointsMetadata())
	{
		SplineMetadata->UpdatePoint(SegmentSplitIndex, t, SplineComp->IsClosedLoop());
	}

	SplineComp->UpdateSpline();
	SplineComp->bSplineHasBeenEdited = true;

	// Transform the spline keys using an EPropertyChangeType::Interactive change. Later on, at the end of mouse tracking, a non-interactive change will be notified via void TrackingStopped :
	NotifyPropertiesModified(SplineComp, SplineProperties, EPropertyChangeType::Interactive);

	GEditor->RedrawLevelEditingViewports(true);
}

void FSplineComponentVisualizer::AddSegment(const FVector& InWorldPos, bool bAppend)
{
	USplineComponent* SplineComp = GetEditedSplineComponent();
	check(SplineComp != nullptr);

	SplineComp->Modify();
	if (AActor* Owner = SplineComp->GetOwner())
	{
		Owner->Modify();
	}

	int32 KeyIdx = 0;
	int32 NewKeyIdx = 0;

	if (bAppend)
	{
		NewKeyIdx = SplineComp->GetNumberOfSplinePoints();
		KeyIdx = NewKeyIdx - 1;
	}
	
	// Set adjacent point to CurveAuto so its tangent adjusts automatically as new point moves.
	if (ConvertSplinePointTypeToInterpCurveMode(SplineComp->GetSplinePointType(KeyIdx)) == CIM_CurveUser)
	{
		SplineComp->SetSplinePointType(KeyIdx, ConvertInterpCurveModeToSplinePointType(CIM_CurveAuto), false);
	}

	// Compute local pos
	const FVector LocalPos = SplineComp->GetComponentTransform().InverseTransformPosition(InWorldPos);

	// Must be saved before adding point so that KeyIdx remains valid while we use it to read from the existing data.
	FQuat NewRot = SplineComp->GetQuaternionAtSplinePoint(KeyIdx, ESplineCoordinateSpace::Local);
	FVector NewScale = SplineComp->GetScaleAtSplinePoint(KeyIdx);
	ESplinePointType::Type NewType = SplineComp->GetSplinePointType(KeyIdx);

	SplineComp->AddSplinePointAtIndex(LocalPos, NewKeyIdx, ESplineCoordinateSpace::Local, false);
	SplineComp->SetQuaternionAtSplinePoint(NewKeyIdx, NewRot, ESplineCoordinateSpace::Local, false);
	SplineComp->SetScaleAtSplinePoint(NewKeyIdx, NewScale, false);
	SplineComp->SetSplinePointType(NewKeyIdx, NewType, false);

	// Set selection to key
	ChangeSelectionState(NewKeyIdx, false);

	SplineComp->UpdateSpline();
	SplineComp->bSplineHasBeenEdited = true;

	NotifyPropertiesModified(SplineComp, SplineProperties);

	GEditor->RedrawLevelEditingViewports(true);
}

void FSplineComponentVisualizer::UpdateAddSegment(const FVector& InDrag)
{
	USplineComponent* SplineComp = GetEditedSplineComponent();
	check(SplineComp != nullptr);
	check(SelectionState);
	const TSet<int32>& SelectedKeys = SelectionState->GetSelectedKeys();
	int32 LastKeyIndexSelected = SelectionState->GetVerifiedLastKeyIndexSelected(SplineComp->GetNumberOfSplinePoints());
	check(SelectedKeys.Num() == 1);
	check(SelectedKeys.Contains(LastKeyIndexSelected));
	// Only work on keys at either end of a non-closed-looped spline 
	check(!SplineComp->IsClosedLoop());
	check(LastKeyIndexSelected == 0 || LastKeyIndexSelected == SplineComp->GetNumberOfSplinePoints() - 1);

	SplineComp->Modify();
	if (AActor* Owner = SplineComp->GetOwner())
	{
		Owner->Modify();
	}

	// Move added point to new position
	const FVector CurrentWorldPos = SplineComp->GetLocationAtSplinePoint(LastKeyIndexSelected, ESplineCoordinateSpace::World);
	const FVector NewWorldPos = CurrentWorldPos + InDrag;
	SplineComp->SetLocationAtSplinePoint(LastKeyIndexSelected, NewWorldPos, ESplineCoordinateSpace::World);

	SplineComp->UpdateSpline();
	SplineComp->bSplineHasBeenEdited = true;

	// Transform the spline keys using an EPropertyChangeType::Interactive change. Later on, at the end of mouse tracking, a non-interactive change will be notified via void TrackingStopped :
	NotifyPropertiesModified(SplineComp, SplineProperties, EPropertyChangeType::Interactive);

	GEditor->RedrawLevelEditingViewports(true);
}

void FSplineComponentVisualizer::ResetAllowDuplication()
{
	bAllowDuplication = true;
	bDuplicatingSplineKey = false;
	bUpdatingAddSegment = false;
	DuplicateDelay = 0;
	DuplicateDelayAccumulatedDrag = FVector::ZeroVector;
	DuplicateCacheSplitSegmentParam = 0.0f;
}

void FSplineComponentVisualizer::OnDeleteKey()
{
	const FScopedTransaction Transaction(LOCTEXT("DeleteSplinePoint", "Delete Spline Point"));

	ResetTempModes();

	USplineComponent* SplineComp = GetEditedSplineComponent();
	check(SplineComp != nullptr);
	check(SelectionState);
	const TSet<int32>& SelectedKeys = SelectionState->GetSelectedKeys();
	int32 LastKeyIndexSelected = SelectionState->GetVerifiedLastKeyIndexSelected(SplineComp->GetNumberOfSplinePoints());
	check(SelectedKeys.Num() > 0);
	check(SelectedKeys.Contains(LastKeyIndexSelected));

	SplineComp->Modify();
	if (AActor* Owner = SplineComp->GetOwner())
	{
		Owner->Modify();
	}

	// Get a sorted list of all the selected indices, highest to lowest
	TArray<int32> SelectedKeysSorted;
	for (int32 SelectedKeyIndex : SelectedKeys)
	{
		SelectedKeysSorted.Add(SelectedKeyIndex);
	}
	SelectedKeysSorted.Sort([](int32 A, int32 B) { return A > B; });

	// Delete selected keys from list, highest index first
	for (int32 SelectedKeyIndex : SelectedKeysSorted)
	{
		SplineComp->RemoveSplinePoint(SelectedKeyIndex, false);
	}

	// Select the first valid key before the deleted range
	const int32 LastKeyIndex = SelectedKeysSorted.Last();
	ChangeSelectionState(LastKeyIndex > 1 ? LastKeyIndex - 1 : 0, false);
	SelectionState->Modify();
	SelectionState->ClearSelectedSegmentIndex();
	SelectionState->ClearSelectedTangentHandle();

	SplineComp->UpdateSpline();
	SplineComp->bSplineHasBeenEdited = true;

	NotifyPropertiesModified(SplineComp, SplineProperties);
	if (AActor* OwnerActor = SplineComp->GetOwner())
	{
		OwnerActor->PostEditMove(true);
	}

	SelectionState->SetCachedRotation(SplineComp->GetQuaternionAtSplinePoint(SelectionState->GetLastKeyIndexSelected(), ESplineCoordinateSpace::World));

	GEditor->RedrawLevelEditingViewports(true);
}


bool FSplineComponentVisualizer::CanDeleteKey() const
{
	USplineComponent* SplineComp = GetEditedSplineComponent();
	const TSet<int32>& SelectedKeys = SelectionState->GetSelectedKeys();
	int32 LastKeyIndexSelected = SelectionState->GetLastKeyIndexSelected();
	return (SplineComp != nullptr &&
			SelectedKeys.Num() > 0 &&
			SelectedKeys.Num() != SplineComp->GetNumberOfSplinePoints() &&
			LastKeyIndexSelected != INDEX_NONE);
}


bool FSplineComponentVisualizer::IsKeySelectionValid() const
{
	USplineComponent* SplineComp = GetEditedSplineComponent();
	const TSet<int32>& SelectedKeys = SelectionState->GetSelectedKeys();
	int32 LastKeyIndexSelected = SelectionState->GetLastKeyIndexSelected();
	return (SplineComp != nullptr &&
			SelectedKeys.Num() > 0 &&
			LastKeyIndexSelected != INDEX_NONE);
}

void FSplineComponentVisualizer::OnLockAxis(EAxis::Type InAxis)
{
	const FScopedTransaction Transaction(LOCTEXT("LockAxis", "Lock Axis"));

	ResetTempModes();

	AddKeyLockedAxis = InAxis;
}

bool FSplineComponentVisualizer::IsLockAxisSet(EAxis::Type Index) const
{
	return (Index == AddKeyLockedAxis);
}

void FSplineComponentVisualizer::OnResetToAutomaticTangent(EInterpCurveMode Mode)
{
	const FScopedTransaction Transaction(LOCTEXT("ResetToAutomaticTangent", "Reset to Automatic Tangent"));

	ResetTempModes();

	USplineComponent* SplineComp = GetEditedSplineComponent();
	if (SplineComp != nullptr)
	{
		check(SelectionState);

		SplineComp->Modify();
		if (AActor* Owner = SplineComp->GetOwner())
		{
			Owner->Modify();
		}

		const TSet<int32>& SelectedKeys = SelectionState->GetSelectedKeys();
		for (int32 SelectedKeyIndex : SelectedKeys)
		{
			if (SplineComponentVisualizerLocals::IsCurvePointType(SplineComp->GetSplinePointType(SelectedKeyIndex)))
			{
				SplineComp->SetSplinePointType(SelectedKeyIndex, ConvertInterpCurveModeToSplinePointType(Mode), false);
			}
		}

		SplineComp->UpdateSpline();
		SplineComp->bSplineHasBeenEdited = true;

		NotifyPropertiesModified(SplineComp, SplineProperties);
		if (AActor* OwnerActor = SplineComp->GetOwner())
		{
			OwnerActor->PostEditMove(true);
		}

		SelectionState->Modify();
		SelectionState->SetCachedRotation(SplineComp->GetQuaternionAtSplinePoint(SelectionState->GetLastKeyIndexSelected(), ESplineCoordinateSpace::World));
	}
}


bool FSplineComponentVisualizer::CanResetToAutomaticTangent(EInterpCurveMode Mode) const
{
	USplineComponent* SplineComp = GetEditedSplineComponent();
	if (SplineComp != nullptr && SelectionState != nullptr && SelectionState->GetLastKeyIndexSelected() != INDEX_NONE)
	{
		const TSet<int32>& SelectedKeys = SelectionState->GetSelectedKeys();
		for (int32 SelectedKeyIndex : SelectedKeys)
		{
			ESplinePointType::Type CurrentMode = SplineComp->GetSplinePointType(SelectedKeyIndex);
			if (SplineComponentVisualizerLocals::IsCurvePointType(CurrentMode) && ConvertSplinePointTypeToInterpCurveMode(CurrentMode) != Mode)
			{
				return true;
			}
		}
	}

	return false;
}


void FSplineComponentVisualizer::OnSetKeyType(EInterpCurveMode Mode)
{
	const FScopedTransaction Transaction(LOCTEXT("SetSplinePointType", "Set Spline Point Type"));

	ResetTempModes();

	USplineComponent* SplineComp = GetEditedSplineComponent();
	if (SplineComp != nullptr)
	{
		check(SelectionState);

		SplineComp->Modify();
		if (AActor* Owner = SplineComp->GetOwner())
		{
			Owner->Modify();
		}

		const TSet<int32>& SelectedKeys = SelectionState->GetSelectedKeys();
		for (int32 SelectedKeyIndex : SelectedKeys)
		{
			check(SelectedKeyIndex >= 0);
			check(SelectedKeyIndex < SplineComp->GetNumberOfSplinePoints());
			SplineComp->SetSplinePointType(SelectedKeyIndex, ConvertInterpCurveModeToSplinePointType(Mode), false);
		}

		SplineComp->UpdateSpline();
		SplineComp->bSplineHasBeenEdited = true;

		NotifyPropertiesModified(SplineComp, SplineProperties);
		if (AActor* OwnerActor = SplineComp->GetOwner())
		{
			OwnerActor->PostEditMove(true);
		}

		SelectionState->Modify();
		SelectionState->SetCachedRotation(SplineComp->GetQuaternionAtSplinePoint(SelectionState->GetLastKeyIndexSelected(), ESplineCoordinateSpace::World));
	}
}


bool FSplineComponentVisualizer::IsKeyTypeSet(EInterpCurveMode Mode) const
{
	if (IsKeySelectionValid())
	{
		USplineComponent* SplineComp = GetEditedSplineComponent();
		check(SplineComp != nullptr);
		check(SelectionState);

		const TSet<int32>& SelectedKeys = SelectionState->GetSelectedKeys();
		for (int32 SelectedKeyIndex : SelectedKeys)
		{
			check(SelectedKeyIndex >= 0);
			check(SelectedKeyIndex < SplineComp->GetNumberOfSplinePoints());
			const ESplinePointType::Type SelectedPointCurveType = SplineComp->GetSplinePointType(SelectedKeyIndex);
			if ((Mode == CIM_CurveAuto && SplineComponentVisualizerLocals::IsCurvePointType(SelectedPointCurveType)) || SelectedPointCurveType == ConvertInterpCurveModeToSplinePointType(Mode))
			{
				return true;
			}
		}
	}

	return false;
}


void FSplineComponentVisualizer::OnSetVisualizeRollAndScale()
{
	ResetTempModes();

	USplineComponent* SplineComp = GetEditedSplineComponent();
	check(SplineComp != nullptr);

	SplineComp->Modify();
	if (AActor* Owner = SplineComp->GetOwner())
	{
		Owner->Modify();
	}

	SplineComp->bShouldVisualizeScale = !SplineComp->bShouldVisualizeScale;

	NotifyPropertyModified(SplineComp, FindFProperty<FProperty>(USplineComponent::StaticClass(), GET_MEMBER_NAME_CHECKED(USplineComponent, bShouldVisualizeScale)));

	GEditor->RedrawLevelEditingViewports(true);
}


bool FSplineComponentVisualizer::IsVisualizingRollAndScale() const
{
	USplineComponent* SplineComp = GetEditedSplineComponent();
	
	return SplineComp ? SplineComp->bShouldVisualizeScale : false;
}


void FSplineComponentVisualizer::OnSetDiscontinuousSpline()
{
	ResetTempModes();

	USplineComponent* SplineComp = GetEditedSplineComponent();
	check(SplineComp != nullptr);

	SplineComp->Modify();
	if (AActor* Owner = SplineComp->GetOwner())
	{
		Owner->Modify();
	}

	SplineComp->bAllowDiscontinuousSpline = !SplineComp->bAllowDiscontinuousSpline;

	// If not allowed discontinuous splines, set all ArriveTangents to match LeaveTangents
	if (!SplineComp->bAllowDiscontinuousSpline)
	{
		for (int Index = 0; Index < SplineComp->GetNumberOfSplinePoints(); Index++)
		{
			SplineComp->SetTangentAtSplinePoint(Index, SplineComp->GetLeaveTangentAtSplinePoint(Index, ESplineCoordinateSpace::Local), ESplineCoordinateSpace::Local, false);
		}
	}

	TArray<FProperty*> Properties = SplineProperties;
	Properties.Add(FindFProperty<FProperty>(USplineComponent::StaticClass(), GET_MEMBER_NAME_CHECKED(USplineComponent, bAllowDiscontinuousSpline)));
	NotifyPropertiesModified(SplineComp, Properties);

	GEditor->RedrawLevelEditingViewports(true);
}


bool FSplineComponentVisualizer::IsDiscontinuousSpline() const
{
	USplineComponent* SplineComp = GetEditedSplineComponent();

	return SplineComp ? SplineComp->bAllowDiscontinuousSpline : false;
}


void FSplineComponentVisualizer::OnToggleClosedLoop()
{
	const FScopedTransaction Transaction(LOCTEXT("ToggleClosedLoop", "Toggle Closed Loop"));

	USplineComponent* SplineComp = GetEditedSplineComponent();
	check(SplineComp != nullptr);

	SplineComp->Modify();
	if (AActor* Owner = SplineComp->GetOwner())
	{
		Owner->Modify();
	}

	SplineComp->SetClosedLoop(!SplineComp->IsClosedLoop());

	TArray<FProperty*> Properties = SplineProperties;
	Properties.Add(FindFProperty<FProperty>(USplineComponent::StaticClass(), GET_MEMBER_NAME_CHECKED(USplineComponent, bClosedLoop)));
	NotifyPropertiesModified(SplineComp, Properties);

	GEditor->RedrawLevelEditingViewports(true);
}

bool FSplineComponentVisualizer::IsClosedLoop() const
{
	USplineComponent* SplineComp = GetEditedSplineComponent();
	return SplineComp ? SplineComp->IsClosedLoop() : false;
}

void FSplineComponentVisualizer::OnResetToDefault()
{
	const FScopedTransaction Transaction(LOCTEXT("ResetToDefault", "Reset to Default"));

	ResetTempModes();

	USplineComponent* SplineComp = GetEditedSplineComponent();
	check(SplineComp != nullptr);
	check(SelectionState);

	SplineComp->Modify();
	SplineComp->ResetToDefault();

	if (AActor* Owner = SplineComp->GetOwner())
	{
		Owner->Modify();
	}

	SplineComp->bSplineHasBeenEdited = false;

	// Select first key
	ChangeSelectionState(0, false);
	SelectionState->Modify();
	SelectionState->ClearSelectedSegmentIndex();
	SelectionState->ClearSelectedTangentHandle();

	if (AActor* Owner = SplineComp->GetOwner())
	{
		Owner->PostEditMove(true);
	}

	GEditor->RedrawLevelEditingViewports(true);
}


bool FSplineComponentVisualizer::CanResetToDefault() const
{
	USplineComponent* SplineComp = GetEditedSplineComponent();
	if(SplineComp != nullptr)
    {
        return SplineComp->CanResetToDefault();
    }
    else
    {
        return false;
    }
}

void FSplineComponentVisualizer::OnAddAttributeKey()
{
	const FScopedTransaction Transaction(LOCTEXT("AddAttributePoint", "Add Attribute Point"));

	ResetTempModes();

	USplineComponent* SplineComp = GetEditedSplineComponent();
	check(SplineComp != nullptr);
	check(SelectionState);

	SelectionState->Modify();

	SplineComp->Modify();
	if (AActor* Owner = SplineComp->GetOwner())
	{
		Owner->Modify();
	}
	
	const float NearestParam = SplineComp->FindInputKeyClosestToWorldLocation(SelectionState->GetSelectedSplinePosition());
	const float CurrentValue = SplineComp->GetFloatPropertyAtSplineInputKey(NearestParam, SelectionState->GetSelectedAttributeName());
	const int32 NewAttributeIndex = SplineComp->AddFloatPropertyAtSplineInputKey(NearestParam, CurrentValue, SelectionState->GetSelectedAttributeName());

	// Update selection
	if (NewAttributeIndex != INDEX_NONE)
	{
		SelectionState->SetSelectedSegmentIndex(INDEX_NONE);
		SelectionState->SetSelectedAttribute(NewAttributeIndex, SelectionState->GetSelectedAttributeName());
		SelectionState->SetSelectedSplinePosition(FVector::ZeroVector);
		SelectionState->SetCachedRotation(SplineComp->GetQuaternionAtSplineInputKey(NearestParam, ESplineCoordinateSpace::World));
	}

	SplineComp->bSplineHasBeenEdited = true;
	NotifyPropertiesModified(SplineComp, SplineProperties, EPropertyChangeType::ArrayAdd);
}

bool FSplineComponentVisualizer::CanAddAttributeKey() const
{
	check (SelectionState);
	const USplineComponent* SplineComp = GetEditedSplineComponent();
	return SplineComp && SplineComp->SupportsAttributes() && SelectionState->GetSelectedAttributeIndex() == INDEX_NONE && SelectionState->GetSelectedAttributeName() != NAME_None;
}

void FSplineComponentVisualizer::OnDeleteAttributeKey()
{
	const FScopedTransaction Transaction(LOCTEXT("RemoveAttributePoint", "Remove Attribute Point"));

	ResetTempModes();

	USplineComponent* SplineComp = GetEditedSplineComponent();
	check(SplineComp != nullptr);
	check(SelectionState);

	SelectionState->Modify();

	SplineComp->Modify();
	if (AActor* Owner = SplineComp->GetOwner())
	{
		Owner->Modify();
	}

	SplineComp->RemovePropertyAtIndex(SelectionState->GetSelectedAttributeIndex(), SelectionState->GetSelectedAttributeName());

	// Update selection
	if (const int32 NumAttributeValues = SplineComp->GetNumberOfPropertyValues(SelectionState->GetSelectedAttributeName()); NumAttributeValues < 1)
	{
		// Clear attribute selection & select a control point
		ChangeSelectionState(0, false);
		SelectionState->SetCachedRotation(SplineComp->GetQuaternionAtSplinePoint(SelectionState->GetLastKeyIndexSelected(), ESplineCoordinateSpace::World));
	}
	else
	{
		// Update attribute selection
		SelectionState->SetSelectedAttribute(FMath::Clamp(SelectionState->GetSelectedAttributeIndex(), 0, NumAttributeValues - 1), SelectionState->GetSelectedAttributeName());
		const float NewParam = SplineComp->GetFloatPropertyInputKeyAtIndex(SelectionState->GetSelectedAttributeIndex(), SelectionState->GetSelectedAttributeName());
		SelectionState->SetCachedRotation(SplineComp->GetQuaternionAtSplineInputKey(NewParam, ESplineCoordinateSpace::World));
	}

	SplineComp->bSplineHasBeenEdited = true;
	NotifyPropertiesModified(SplineComp, SplineProperties, EPropertyChangeType::ArrayRemove);
	
	// Necessary because the attribute proxies store invalid indices, we must redraw them.
	GEditor->RedrawAllViewports();
}

bool FSplineComponentVisualizer::CanDeleteAttributeKey() const
{
	check (SelectionState);
	const USplineComponent* SplineComp = GetEditedSplineComponent();
	return SplineComp && SplineComp->SupportsAttributes() && SelectionState->GetSelectedAttributeIndex() != INDEX_NONE && SelectionState->GetSelectedAttributeName() != NAME_None;
}

bool FSplineComponentVisualizer::HandleSelectFirstLastSplinePoint(USplineComponent* InSplineComponent, bool bFirstPoint)
{
	const FScopedTransaction Transaction(LOCTEXT("SelectFirstSplinePoint", "Select First Spline Point"));

	check(InSplineComponent);
	check(SelectionState);

	bool bResetEditedSplineComponent = false;
	if (GetEditedSplineComponent() != InSplineComponent)
	{
		SetEditedSplineComponent(InSplineComponent);
		bResetEditedSplineComponent = true;
	}

	OnSelectFirstLastSplinePoint(bFirstPoint);

	return bResetEditedSplineComponent;
}

void FSplineComponentVisualizer::HandleSelectPrevNextSplinePoint(bool bNext, bool bAddToSelection)
{
	OnSelectPrevNextSplinePoint(bNext, bAddToSelection);
}

bool FSplineComponentVisualizer::HandleSelectAllSplinePoints(USplineComponent* InSplineComponent)
{
	const FScopedTransaction Transaction(LOCTEXT("SelectAllSplinePoints", "Select All Spline Points"));

	check(InSplineComponent);
	check(SelectionState);

	bool bResetEditedSplineComponent = false;
	if (GetEditedSplineComponent() != InSplineComponent)
	{
		SetEditedSplineComponent(InSplineComponent);
		bResetEditedSplineComponent = true;
	}

	OnSelectAllSplinePoints();

	return bResetEditedSplineComponent;
}

void FSplineComponentVisualizer::OnSelectFirstLastSplinePoint(bool bFirstPoint)
{
	const FScopedTransaction Transaction(LOCTEXT("SelectFirstSplinePoint", "Select First Spline Point"));

	ResetTempModes();

	if (USplineComponent* SplineComp = GetEditedSplineComponent())
	{
		const int32 NumSplinePoints = SplineComp->GetNumberOfSplinePoints();
		if (NumSplinePoints > 0)
		{
			SelectSplinePoint(bFirstPoint ? 0 : NumSplinePoints - 1, false);
		}
	}
}

void FSplineComponentVisualizer::OnSelectPrevNextSplinePoint(bool bNextPoint, bool bAddToSelection)
{
	const FScopedTransaction Transaction(LOCTEXT("SelectSplinePoint", "Select Spline Point"));

	ResetTempModes();

	if (USplineComponent* SplineComp = GetEditedSplineComponent())
	{
		if (AreKeysSelected())
		{
			const int32 NumSplinePoints = SplineComp->GetNumberOfSplinePoints();
			check(SelectionState);
			const int32 LastKeyIndexSelected = SelectionState->GetVerifiedLastKeyIndexSelected(NumSplinePoints);

			int32 SelectIndex = INDEX_NONE;
			const int32 Step = bNextPoint ? 1 : -1;
			auto WrapKeys = [&NumSplinePoints](int32 Key) { return (Key >= NumSplinePoints ? 0 : (Key < 0 ? NumSplinePoints - 1 : Key)); };

			for (int32 Index = WrapKeys(LastKeyIndexSelected + Step); Index != LastKeyIndexSelected; Index = WrapKeys(Index + Step))
			{
				if (!bAddToSelection || !SelectionState->IsSplinePointSelected(Index))
				{
					SelectIndex = Index;
					break;
				}
			}

			if (SelectIndex != INDEX_NONE)
			{
				if (!bAddToSelection)
				{	
					SelectSplinePoint(SelectIndex, false);
				}
				else
				{
					// To do: change the following to use SelectSplinePoint(), with a parameter bClearMetadataSelectionState set to false.
					check(SelectionState);
					SelectionState->Modify();

					TSet<int32>& SelectedKeys = SelectionState->ModifySelectedKeys();
					SelectedKeys.Add(SelectIndex);

					SelectionState->SetLastKeyIndexSelected(SelectIndex);
					SelectionState->ClearSelectedSegmentIndex();
					SelectionState->ClearSelectedTangentHandle();
					SelectionState->SetCachedRotation(SplineComp->GetQuaternionAtSplinePoint(SelectionState->GetLastKeyIndexSelected(), ESplineCoordinateSpace::World));

					GEditor->RedrawLevelEditingViewports(true);
				}
			}
		}
	}
}

void FSplineComponentVisualizer::SetCachedRotation(const FQuat& NewRotation)
{
	check(SelectionState);
	SelectionState->Modify();
	SelectionState->SetCachedRotation(NewRotation);
}

void FSplineComponentVisualizer::SelectSplinePoint(int32 SelectIndex, bool bAddToSelection)
{
	const FScopedTransaction Transaction(LOCTEXT("SelectSplinePoint", "Select Spline Point"));

	ResetTempModes();

	check(SelectionState);

	if (USplineComponent* SplineComp = GetEditedSplineComponent())
	{
		if (SelectIndex != INDEX_NONE)
		{
			SelectionState->Modify();

			ChangeSelectionState(SelectIndex, bAddToSelection);

			SelectionState->ClearSelectedSegmentIndex();
			SelectionState->ClearSelectedTangentHandle();
			SelectionState->SetCachedRotation(SplineComp->GetQuaternionAtSplinePoint(SelectionState->GetLastKeyIndexSelected(), ESplineCoordinateSpace::World));

			GEditor->RedrawLevelEditingViewports(true);
		}
	}
}

void FSplineComponentVisualizer::OnSelectAllSplinePoints()
{
	const FScopedTransaction Transaction(LOCTEXT("SelectAllSplinePoints", "Select All Spline Points"));

	ResetTempModes();

	if (USplineComponent* SplineComp = GetEditedSplineComponent())
	{
		int32 NumPoints = SplineComp->GetNumberOfSplinePoints();

		check(SelectionState);
		SelectionState->Modify();

		TSet<int32>& SelectedKeys = SelectionState->ModifySelectedKeys();
		SelectedKeys.Empty();

		// Spline control point selection always uses transparent box selection.
		for (int32 KeyIdx = 0; KeyIdx < NumPoints; KeyIdx++)
		{
			SelectedKeys.Add(KeyIdx);
		}

		SelectionState->SetLastKeyIndexSelected(NumPoints - 1);
		SelectionState->ClearSelectedSegmentIndex();
		SelectionState->ClearSelectedTangentHandle();
		SelectionState->SetCachedRotation(SplineComp->GetQuaternionAtSplinePoint(SelectionState->GetLastKeyIndexSelected(), ESplineCoordinateSpace::World));

		GEditor->RedrawLevelEditingViewports(true);
	}
}

bool FSplineComponentVisualizer::CanSelectSplinePoints() const
{
	USplineComponent* SplineComp = GetEditedSplineComponent();
	return (SplineComp != nullptr);
}

TSharedPtr<SWidget> FSplineComponentVisualizer::GenerateContextMenu() const
{
	if (const USplineComponent* SplineComp = GetEditedSplineComponent();
		SplineComp && !SplineComponentVisualizerLocals::IsEnabledForSpline(SplineComp))
	{
		return nullptr;
	}
	
	FMenuBuilder MenuBuilder(true, SplineComponentVisualizerActions);
	
	GenerateContextMenuSections(MenuBuilder);

	TSharedPtr<SWidget> MenuWidget = MenuBuilder.MakeWidget();
	return MenuWidget;
}

void FSplineComponentVisualizer::GenerateContextMenuSections(FMenuBuilder& InMenuBuilder) const
{
	InMenuBuilder.BeginSection("SplinePointEdit", LOCTEXT("SplinePoint", "Spline Point"));

	USplineComponent* SplineComp = GetEditedSplineComponent();
	if (SplineComp != nullptr)
	{
		check(SelectionState);

		if (SelectionState->GetSelectedSegmentIndex() != INDEX_NONE)
		{
			InMenuBuilder.AddMenuEntry(FSplineComponentVisualizerCommands::Get().AddKey);
		}
		else if (SelectionState->GetLastKeyIndexSelected() != INDEX_NONE)
		{
			InMenuBuilder.AddMenuEntry(FSplineComponentVisualizerCommands::Get().DeleteKey);
			InMenuBuilder.AddMenuEntry(FSplineComponentVisualizerCommands::Get().DuplicateKey);

			InMenuBuilder.AddSubMenu(
				LOCTEXT("SelectSplinePoints", "Select Spline Points"),
				LOCTEXT("SelectSplinePointsTooltip", "Select spline point."),
				FNewMenuDelegate::CreateSP(this, &FSplineComponentVisualizer::GenerateSelectSplinePointsSubMenu));

			InMenuBuilder.AddSubMenu(
				LOCTEXT("SplinePointType", "Spline Point Type"),
				LOCTEXT("SplinePointTypeTooltip", "Define the type of the spline point."),
				FNewMenuDelegate::CreateSP(this, &FSplineComponentVisualizer::GenerateSplinePointTypeSubMenu));

			// Only add the Automatic Tangents submenu if any of the keys is a curve type
			const TSet<int32>& SelectedKeys = SelectionState->GetSelectedKeys();
			for (int32 SelectedKeyIndex : SelectedKeys)
			{
				check(SelectedKeyIndex >= 0);
				check(SelectedKeyIndex < SplineComp->GetNumberOfSplinePoints());
				if (SplineComponentVisualizerLocals::IsCurvePointType(SplineComp->GetSplinePointType(SelectedKeyIndex)))
				{
					InMenuBuilder.AddSubMenu(
						LOCTEXT("ResetToAutomaticTangent", "Reset to Automatic Tangent"),
						LOCTEXT("ResetToAutomaticTangentTooltip", "Reset the spline point tangent to an automatically generated value."),
						FNewMenuDelegate::CreateSP(this, &FSplineComponentVisualizer::GenerateTangentTypeSubMenu));
					break;
				}
			}

			InMenuBuilder.AddMenuEntry(
				LOCTEXT("SplineGenerate", "Spline Generation Panel"),
				LOCTEXT("SplineGenerateTooltip", "Opens up a spline generation panel to easily create basic shapes with splines"),
				FSlateIcon(),
				FUIAction( 
					FExecuteAction::CreateSP(const_cast<FSplineComponentVisualizer*>(this), &FSplineComponentVisualizer::CreateSplineGeneratorPanel),
					FCanExecuteAction::CreateLambda([] { return true; })
				)
			);
		}
	}
	InMenuBuilder.EndSection();

	InMenuBuilder.BeginSection("Transform");
	{
		InMenuBuilder.AddMenuEntry(FSplineComponentVisualizerCommands::Get().FocusViewportToSelection);

		InMenuBuilder.AddSubMenu(
			LOCTEXT("SplineSnapAlign", "Snap/Align"),
			LOCTEXT("SplineSnapAlignTooltip", "Snap align options."),
			FNewMenuDelegate::CreateSP(this, &FSplineComponentVisualizer::GenerateSnapAlignSubMenu));

		/* temporarily disabled
		InMenuBuilder.AddSubMenu(
			LOCTEXT("LockAxis", "Lock Axis"),
			LOCTEXT("LockAxisTooltip", "Axis to lock when adding new spline points."),
			FNewMenuDelegate::CreateSP(this, &FSplineComponentVisualizer::GenerateLockAxisSubMenu));
			*/
	}
	InMenuBuilder.EndSection();

	InMenuBuilder.BeginSection("Spline", LOCTEXT("Spline", "Spline"));
	{
		InMenuBuilder.AddMenuEntry(FSplineComponentVisualizerCommands::Get().ToggleClosedLoop);
		InMenuBuilder.AddMenuEntry(FSplineComponentVisualizerCommands::Get().ResetToDefault);
	}
	InMenuBuilder.EndSection();

	InMenuBuilder.BeginSection("Visualization", LOCTEXT("Visualization", "Visualization"));
	{
		InMenuBuilder.AddMenuEntry(FSplineComponentVisualizerCommands::Get().VisualizeRollAndScale);
		InMenuBuilder.AddMenuEntry(FSplineComponentVisualizerCommands::Get().DiscontinuousSpline);
	}
	InMenuBuilder.EndSection();

	if (SplineComp && SplineComp->SupportsAttributes())
	{
		GenerateAttributeMenuSection(InMenuBuilder);
	}
}

void FSplineComponentVisualizer::GenerateSelectSplinePointsSubMenu(FMenuBuilder& MenuBuilder) const
{
	MenuBuilder.AddMenuEntry(FSplineComponentVisualizerCommands::Get().SelectAll);
	MenuBuilder.AddMenuEntry(FSplineComponentVisualizerCommands::Get().SelectPrevSplinePoint);
	MenuBuilder.AddMenuEntry(FSplineComponentVisualizerCommands::Get().SelectNextSplinePoint);
	MenuBuilder.AddMenuEntry(FSplineComponentVisualizerCommands::Get().AddPrevSplinePoint);
	MenuBuilder.AddMenuEntry(FSplineComponentVisualizerCommands::Get().AddNextSplinePoint);
}
void FSplineComponentVisualizer::GenerateSplinePointTypeSubMenu(FMenuBuilder& MenuBuilder) const
{
	if (const USplineComponent* SplineComp = GetEditedSplineComponent())
	{
		const TArray<ESplinePointType::Type> EnabledSplinePointTypes = SplineComp->GetEnabledSplinePointTypes();
		if (EnabledSplinePointTypes.Contains(ESplinePointType::Curve))
		{
			MenuBuilder.AddMenuEntry(FSplineComponentVisualizerCommands::Get().SetKeyToCurve);
		}
		if (EnabledSplinePointTypes.Contains(ESplinePointType::Linear))
		{
			MenuBuilder.AddMenuEntry(FSplineComponentVisualizerCommands::Get().SetKeyToLinear);
		}
		if (EnabledSplinePointTypes.Contains(ESplinePointType::Constant))
		{
			MenuBuilder.AddMenuEntry(FSplineComponentVisualizerCommands::Get().SetKeyToConstant);
		}
	}
}

void FSplineComponentVisualizer::GenerateTangentTypeSubMenu(FMenuBuilder& MenuBuilder) const
{
	MenuBuilder.AddMenuEntry(FSplineComponentVisualizerCommands::Get().ResetToUnclampedTangent);
	MenuBuilder.AddMenuEntry(FSplineComponentVisualizerCommands::Get().ResetToClampedTangent);
}

void FSplineComponentVisualizer::GenerateSnapAlignSubMenu(FMenuBuilder& MenuBuilder) const
{
	MenuBuilder.AddMenuEntry(FLevelEditorCommands::Get().SnapToFloor);
	MenuBuilder.AddMenuEntry(FLevelEditorCommands::Get().AlignToFloor);
	MenuBuilder.AddSeparator();
	MenuBuilder.AddMenuEntry(FSplineComponentVisualizerCommands::Get().SnapKeyToNearestSplinePoint);
	MenuBuilder.AddMenuEntry(FSplineComponentVisualizerCommands::Get().AlignKeyToNearestSplinePoint);
	MenuBuilder.AddMenuEntry(FSplineComponentVisualizerCommands::Get().AlignKeyPerpendicularToNearestSplinePoint);
	MenuBuilder.AddSeparator();
	MenuBuilder.AddMenuEntry(FSplineComponentVisualizerCommands::Get().SnapKeyToActor);
	MenuBuilder.AddMenuEntry(FSplineComponentVisualizerCommands::Get().AlignKeyToActor);
	MenuBuilder.AddMenuEntry(FSplineComponentVisualizerCommands::Get().AlignKeyPerpendicularToActor);
	MenuBuilder.AddSeparator();

	if (AxisDisplayInfo::GetAxisDisplayCoordinateSystem() == EAxisList::LeftUpForward)
	{
		MenuBuilder.AddMenuEntry(FSplineComponentVisualizerCommands::Get().SnapAllToSelectedY);
		MenuBuilder.AddMenuEntry(FSplineComponentVisualizerCommands::Get().SnapAllToSelectedZ);
		MenuBuilder.AddMenuEntry(FSplineComponentVisualizerCommands::Get().SnapAllToSelectedX);
		MenuBuilder.AddSeparator();
		MenuBuilder.AddMenuEntry(FSplineComponentVisualizerCommands::Get().SnapToLastSelectedY);
		MenuBuilder.AddMenuEntry(FSplineComponentVisualizerCommands::Get().SnapToLastSelectedZ);
		MenuBuilder.AddMenuEntry(FSplineComponentVisualizerCommands::Get().SnapToLastSelectedX);
	}
	else
	{
		MenuBuilder.AddMenuEntry(FSplineComponentVisualizerCommands::Get().SnapAllToSelectedX);
		MenuBuilder.AddMenuEntry(FSplineComponentVisualizerCommands::Get().SnapAllToSelectedY);
		MenuBuilder.AddMenuEntry(FSplineComponentVisualizerCommands::Get().SnapAllToSelectedZ);
		MenuBuilder.AddSeparator();
		MenuBuilder.AddMenuEntry(FSplineComponentVisualizerCommands::Get().SnapToLastSelectedX);
		MenuBuilder.AddMenuEntry(FSplineComponentVisualizerCommands::Get().SnapToLastSelectedY);
		MenuBuilder.AddMenuEntry(FSplineComponentVisualizerCommands::Get().SnapToLastSelectedZ);
	}

	MenuBuilder.AddSeparator();
	MenuBuilder.AddMenuEntry(FSplineComponentVisualizerCommands::Get().StraightenToNext);
	MenuBuilder.AddMenuEntry(FSplineComponentVisualizerCommands::Get().StraightenToPrevious);
	MenuBuilder.AddSeparator();
	MenuBuilder.AddMenuEntry(FSplineComponentVisualizerCommands::Get().ToggleSnapTangentAdjustments);
}

void FSplineComponentVisualizer::GenerateLockAxisSubMenu(FMenuBuilder& MenuBuilder) const
{
	MenuBuilder.AddMenuEntry(FSplineComponentVisualizerCommands::Get().SetLockedAxisNone);
	if (AxisDisplayInfo::GetAxisDisplayCoordinateSystem() == EAxisList::LeftUpForward)
	{
		MenuBuilder.AddMenuEntry(FSplineComponentVisualizerCommands::Get().SetLockedAxisY);
		MenuBuilder.AddMenuEntry(FSplineComponentVisualizerCommands::Get().SetLockedAxisZ);
		MenuBuilder.AddMenuEntry(FSplineComponentVisualizerCommands::Get().SetLockedAxisX);
	}
	else
	{
		MenuBuilder.AddMenuEntry(FSplineComponentVisualizerCommands::Get().SetLockedAxisX);
		MenuBuilder.AddMenuEntry(FSplineComponentVisualizerCommands::Get().SetLockedAxisY);
		MenuBuilder.AddMenuEntry(FSplineComponentVisualizerCommands::Get().SetLockedAxisZ);
	}
}

void FSplineComponentVisualizer::GenerateAttributeMenuSection(FMenuBuilder& InMenuBuilder) const
{
	InMenuBuilder.BeginSection("Attributes", LOCTEXT("Attributes", "Attributes"));

	InMenuBuilder.AddWidget(GenerateChannelWidget().ToSharedRef(), FText::GetEmpty(), true);
	
	if (SelectionState)
	{
		if (SelectionState->GetSelectedAttributeIndex() == INDEX_NONE)
		{
			InMenuBuilder.AddMenuEntry(FSplineComponentVisualizerCommands::Get().AddAttributeKey);
		}
		else
		{
			InMenuBuilder.AddWidget(GenerateAttributeEditorWidget().ToSharedRef(), FText::GetEmpty(), true);
			InMenuBuilder.AddMenuEntry(FSplineComponentVisualizerCommands::Get().DeleteAttributeKey);
		}
	}

	InMenuBuilder.EndSection();
}

TSharedPtr<SWidget> FSplineComponentVisualizer::GenerateChannelWidget() const
{
	return SNew(SVerticalBox)
		
		+ SVerticalBox::Slot()
		.Padding(10.f, 5.f)
		.AutoHeight()
		[
			GenerateChannelCreatorWidget().ToSharedRef()
		]
    
		+ SVerticalBox::Slot()
		.Padding(10.f, 1.f)
		.AutoHeight()
		[
			GenerateChannelSelectorWidget().ToSharedRef()
		];
}

TSharedPtr<SWidget> FSplineComponentVisualizer::GenerateAttributeEditorWidget() const
{
	// Precompute these values so they can be used as arguments for the widget itself (not just used by the widget lambdas).
	float LowerBound = 0.f;
	float UpperBound = 0.f;
	if (const USplineComponent* SplineComp = GetEditedSplineComponent())
	{
		UpperBound = static_cast<float>(SplineComp->GetNumberOfSplineSegments());
	}
	
	auto GetParameter = [this]() -> TOptional<float>
	{
		if (USplineComponent* LocalSplineComp = GetEditedSplineComponent())
		{
			const int32 AttributeIndex = SelectionState ? SelectionState->GetSelectedAttributeIndex() : INDEX_NONE;
			const FName AttributeName = SelectionState ? SelectionState->GetSelectedAttributeName() : NAME_None;
	
			if (AttributeIndex != INDEX_NONE && AttributeName != NAME_None)
			{
				return LocalSplineComp->GetFloatPropertyInputKeyAtIndex(AttributeIndex, AttributeName);
			}
		}
		
		return TOptional<float>();
	};
	
	auto SetParameter = [this, LowerBound, UpperBound](float NewParameter)
	{
		if (USplineComponent* LocalSplineComp = GetEditedSplineComponent())
		{
			if (SelectionState)
			{
				const int32 AttributeIndex = SelectionState->GetSelectedAttributeIndex();
				const FName AttributeName = SelectionState->GetSelectedAttributeName();
	
				if (AttributeIndex != INDEX_NONE && AttributeName != NAME_None)
				{
					const FScopedTransaction Transaction(LOCTEXT("SetAttributeParameter", "Set Attribute Parameter"));

					LocalSplineComp->Modify();
					if (AActor* Owner = LocalSplineComp->GetOwner())
					{
						Owner->Modify();
					}
					
					const float NewParam = FMath::Clamp(NewParameter, LowerBound, UpperBound);
					SelectionState->SetCachedRotation(LocalSplineComp->GetQuaternionAtSplineInputKey(NewParam, ESplineCoordinateSpace::World));
					if (const int32 NewIndex = LocalSplineComp->SetFloatPropertyInputKeyAtIndex(AttributeIndex, NewParam, AttributeName); NewIndex != AttributeIndex)
					{
						SelectionState->SetSelectedAttribute(NewIndex, SelectionState->GetSelectedAttributeName());
					}

					LocalSplineComp->bSplineHasBeenEdited = true;
					NotifyPropertiesModified(LocalSplineComp, SplineProperties, EPropertyChangeType::ValueSet);
					GEditor->RedrawAllViewports();
				}
			}
		}
	};
	
	auto GetValue = [this]() -> TOptional<float>
	{
		if (USplineComponent* LocalSplineComp = GetEditedSplineComponent())
		{
			const int32 AttributeIndex = SelectionState ? SelectionState->GetSelectedAttributeIndex() : INDEX_NONE;
			const FName AttributeName = SelectionState ? SelectionState->GetSelectedAttributeName() : NAME_None;
	
			if (AttributeIndex != INDEX_NONE && AttributeName != NAME_None)
			{
				return LocalSplineComp->GetFloatPropertyAtIndex(AttributeIndex, AttributeName);
			}
		}

		return TOptional<float>();
	};

	auto SetValue = [this](float NewValue)
	{
		if (USplineComponent* LocalSplineComp = GetEditedSplineComponent())
		{
			const int32 AttributeIndex = SelectionState ? SelectionState->GetSelectedAttributeIndex() : INDEX_NONE;
			const FName AttributeName = SelectionState ? SelectionState->GetSelectedAttributeName() : NAME_None;
	
			if (AttributeIndex != INDEX_NONE && AttributeName != NAME_None)
			{
				const FScopedTransaction Transaction(LOCTEXT("SetAttributeValue", "Set Attribute Value"));

				LocalSplineComp->Modify();
				if (AActor* Owner = LocalSplineComp->GetOwner())
				{
					Owner->Modify();
				}
				
				LocalSplineComp->SetFloatPropertyAtIndex(AttributeIndex, NewValue, AttributeName);
				
				LocalSplineComp->bSplineHasBeenEdited = true;
				NotifyPropertiesModified(LocalSplineComp, SplineProperties, EPropertyChangeType::ValueSet);
				GEditor->RedrawAllViewports();
			}
		}
	};

	TSharedPtr<SVerticalBox> AttributeEditor = SNew(SVerticalBox)

	// Parameter editor
	+ SVerticalBox::Slot()
	.AutoHeight()
	.Padding(10.f, 1.f)
	[
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.Padding(10.f, 1.f)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(FText::FromString(TEXT("Parameter: ")))
			.Justification(ETextJustify::Left)
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(10.f, 1.f)
		[
			SNew(SNumericEntryBox<float>)
			.MinDesiredValueWidth(100.0f)
			.Value_Lambda(GetParameter)
			.OnValueChanged_Lambda(SetParameter)
			.AllowSpin(true)
			.MinValue(LowerBound)
			.MinSliderValue(LowerBound)
			.MaxValue(UpperBound)
			.MaxSliderValue(UpperBound)
		]
	]

	// Value editor
	+ SVerticalBox::Slot()
	.AutoHeight()
	.Padding(10.f, 1.f)
	[
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.Padding(10.f, 1.f)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(FText::FromString(TEXT("Value: ")))
			.Justification(ETextJustify::Left)
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(10.f, 1.f)
		[
			SNew(SNumericEntryBox<float>)
			.MinDesiredValueWidth(100.0f)
			.Value_Lambda(GetValue)
			.OnValueChanged_Lambda(SetValue)
			.AllowSpin(true)
		]
	];

	return AttributeEditor;
}

TSharedPtr<SWidget> FSplineComponentVisualizer::GenerateChannelCreatorWidget() const
{
	TSharedPtr<SEditableTextBox> ChannelNameBox = SNew(SEditableTextBox)
		.HintText(FText::FromString(TEXT("Attribute Name")));

	auto OnCreateChannelPressed = [this](FName Name) -> FReply
	{
		if (USplineComponent* SplineComp = GetEditedSplineComponent())
		{
			SplineComp->CreateFloatPropertyChannel(Name);

			if (SelectionState)
			{
				SelectionState->SetSelectedAttribute(INDEX_NONE, Name);
				GEditor->RedrawAllViewports();
			}
		}

		// Necessary for existing selector widget to show new channel option.
		UpdateSharedAttributeNames();

		return FReply::Handled();
	};

	return SNew(SHorizontalBox)
    
		+ SHorizontalBox::Slot()
		.Padding(10.f, 1.f)
		.FillWidth(1)
		[
			ChannelNameBox.ToSharedRef()
		]
        
		+ SHorizontalBox::Slot()
		.Padding(10.f, 1.f)
		.AutoWidth()
		[
			SNew(SButton)
			.Text(FText::FromString(TEXT("Create")))
			.OnClicked_Lambda([OnCreateChannelPressed, WeakChannelNameBox = ChannelNameBox.ToWeakPtr()]()
			{
				if (TSharedPtr<SEditableTextBox> StrongChannelNameBox = WeakChannelNameBox.Pin())
				{
					FName ChannelName(*StrongChannelNameBox->GetText().ToString());
					StrongChannelNameBox->SetText(FText::GetEmpty());
					return OnCreateChannelPressed(ChannelName);
				}
				
				return FReply::Unhandled();
			})
		];
}

TSharedPtr<SWidget> FSplineComponentVisualizer::GenerateChannelSelectorWidget() const
{
	TSharedPtr<SComboButton> ComboButton = SNew(SComboButton)
		.ButtonContent()
		[
			SNew(STextBlock)
			.Text_Lambda([this]()
			{
				return FText::FromString(SelectionState->GetSelectedAttributeName().ToString());
			})
			.Justification(ETextJustify::Center)
		];

	TSharedPtr<SHorizontalBox> ChannelSelector = SNew(SHorizontalBox)
	
		+ SHorizontalBox::Slot()
		.Padding(10.f, 1.f)
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(FText::FromString(TEXT("Selected Channel: ")))
		]
	
		+ SHorizontalBox::Slot()
		.Padding(10.f, 1.f)
		.FillWidth(1)
		.VAlign(VAlign_Center)
		[
			ComboButton.ToSharedRef()
		];

	auto OnAttributeSelected = [this, WeakCombo = ComboButton.ToWeakPtr()](const TSharedPtr<FName>& SelectedItem, ESelectInfo::Type)
	{
		SelectionState->SetSelectedAttribute(INDEX_NONE, SelectedItem ? *SelectedItem : NAME_None);

		for (TPair<TWeakObjectPtr<const USplineComponent>, FPDICache>& ComponentPDICache : PDICache)
		{
			ComponentPDICache.Value.bDirty = true;
		}
			
		if (TSharedPtr<SComboButton> PinnedComboButton = WeakCombo.Pin())
		{
			PinnedComboButton->SetIsOpen(false);
		}

		// Necessary to cause attribute handles to be rendered before closing the context menu
		GEditor->RedrawAllViewports();
	};

	UpdateSharedAttributeNames();
	
	// We must defer adding menu content til ComboButton is assigned because of the weak ptr capture above.
	// Otherwise, we capture it before it is valid.
	ComboButton->SetMenuContent(SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SListView<TSharedPtr<FName>>)
			.SelectionMode(ESelectionMode::SingleToggle)
			.ListItemsSource(&SharedAttributeNames)
			.OnGenerateRow_Lambda([](TSharedPtr<FName> Item, const TSharedRef<STableViewBase>& OwnerTable)
			{
				return SNew(STableRow<TSharedPtr<FName>>, OwnerTable)
				.Padding(1.f)
				[
					SNew(STextBlock)
					.Text(FText::FromName(*Item))
					.Justification(ETextJustify::Center)
				];
			})
			.OnSelectionChanged_Lambda(OnAttributeSelected)
		]
	);

	return ChannelSelector;
}

void FSplineComponentVisualizer::CreateSplineGeneratorPanel()
{
	SAssignNew(SplineGeneratorPanel, SSplineGeneratorPanel, SharedThis(this));

	TSharedPtr<SWindow> ExistingWindow = WeakExistingWindow.Pin();
	if (!ExistingWindow.IsValid())
	{
		ExistingWindow = SNew(SWindow)
			.ScreenPosition(FSlateApplication::Get().GetCursorPos())
			.Title(LOCTEXT("SplineGenerationPanelTitle", "Spline Generation"))
			.SizingRule(ESizingRule::Autosized)
			.AutoCenter(EAutoCenter::None)
			.SupportsMaximize(false)
			.SupportsMinimize(false);

		ExistingWindow->SetOnWindowClosed(FOnWindowClosed::CreateSP(SplineGeneratorPanel.ToSharedRef(), &SSplineGeneratorPanel::OnWindowClosed));

		TSharedPtr<SWindow> RootWindow = FSlateApplication::Get().GetActiveTopLevelWindow();

		if (RootWindow.IsValid())
		{
			FSlateApplication::Get().AddWindowAsNativeChild(ExistingWindow.ToSharedRef(), RootWindow.ToSharedRef());
		}
		else
		{
			FSlateApplication::Get().AddWindow(ExistingWindow.ToSharedRef());
		}

		ExistingWindow->BringToFront();
		WeakExistingWindow = ExistingWindow;
	}
	else
	{
		ExistingWindow->BringToFront();
	}
	ExistingWindow->SetContent(SplineGeneratorPanel.ToSharedRef());
}

void FSplineComponentVisualizer::OnDeselectedInEditor(TObjectPtr<USplineComponent> SplineComponent)
{
	if (SplineComponentVisualizerLocals::GIgnoreSpuriousDeselect && SplineComponent)
	{
		// Broadcast is for a component we've already moved past. HandleProxyForComponentVis updates our
		// path before queued deselect notifications fire for the previously-edited spline.
		if (SplineComponent != GetEditedSplineComponent())
		{
			return;
		}

		// Component is still in the editor's selection set -- transient deselect-then-reselect cycle.
		if (GEditor && GEditor->GetSelectedComponents()->IsSelected(SplineComponent))
		{
			return;
		}
	}

	if (DeselectedInEditorDelegateHandle.IsValid() && SplineComponent)
	{
		SplineComponent->OnDeselectedInEditor.Remove(DeselectedInEditorDelegateHandle);
	}
	DeselectedInEditorDelegateHandle.Reset();
	EndEditing();
}

bool FSplineComponentVisualizer::ShouldUseForSpline(const USplineComponent* SplineComponent) const
{
	// This implementation should be used when the classic visualizer is enabled for this spline and this is the registered visualizer for the class of the specified spline.
	return GUnrealEd->FindComponentVisualizer(SplineComponent->GetClass()).Get() == this && SplineComponentVisualizerLocals::IsEnabledForSpline(SplineComponent);
}

bool FSplineComponentVisualizer::IsEnabledForSpline(const USplineComponent* InSplineComponent) const
{
	auto ModularFeatureName = ISplineComponentVisualizerSuppressor::GetModularFeatureName();

	if (IModularFeatures::Get().IsModularFeatureAvailable(ModularFeatureName))
	{
		return !IModularFeatures::Get().GetModularFeature<ISplineComponentVisualizerSuppressor>(ModularFeatureName).ShouldSuppress(InSplineComponent);
	}
	
	return true;
}

void FSplineComponentVisualizer::ActivateVisualization()
{
	TSharedPtr<FComponentVisualizer> ComponentVis = StaticCastSharedPtr<FComponentVisualizer>(SharedThis(this).ToSharedPtr());
	GUnrealEd->ComponentVisManager.SetActiveComponentVis(GCurrentLevelEditingViewportClient, ComponentVis);
}

#undef LOCTEXT_NAMESPACE
