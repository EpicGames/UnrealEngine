// Copyright Epic Games, Inc. All Rights Reserved.

#include "SplineGeneratorPanel.h"

#include "ClassViewerModule.h"
#include "ComponentVisualizer.h"
#include "Containers/EnumAsByte.h"
#include "Containers/Set.h"
#include "DetailsViewArgs.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Fonts/SlateFontInfo.h"
#include "GameFramework/Actor.h"
#include "HAL/PlatformCrt.h"
#include "IDetailsView.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Layout/WidgetPath.h"
#include "Math/InterpCurve.h"
#include "Math/InterpCurvePoint.h"
#include "Math/Quat.h"
#include "Math/Rotator.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "Math/VectorRegister.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "ScopedTransaction.h"
#include "SlotBase.h"
#include "SplineComponentVisualizer.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateTypes.h"
#include "Templates/Casts.h"
#include "Types/SlateEnums.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealType.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SplineGeneratorPanel)

class SWindow;
struct FFocusEvent;
struct FGeometry;

#define LOCTEXT_NAMESPACE "SplineGenerator"

void USplineGeneratorBase::Init(TWeakPtr<FSplineComponentVisualizer> InWeakSplineVis)
{
	check(InWeakSplineVis.IsValid());
	WeakSplineVis = InWeakSplineVis;

	TSharedPtr<FSplineComponentVisualizer> SplineVis = WeakSplineVis.Pin();

	SelectedSplineComponent = MakeWeakObjectPtr(SplineVis->GetEditedSplineComponent());

	StartKey = INDEX_NONE;
	check(SplineVis->GetSelectedKeys().Num() == 1);
	StartKey = *SplineVis->GetSelectedKeys().CreateConstIterator();
	CachedSplineCurves = SelectedSplineComponent->GetSplineCurves();
}

void USplineGeneratorBase::Reset()
{
	if (!SelectedSplineComponent.IsValid())
	{
		TSharedPtr<FSplineComponentVisualizer> SplineVis = WeakSplineVis.Pin();
		check(SplineVis);

		SelectedSplineComponent = MakeWeakObjectPtr(SplineVis->GetEditedSplineComponent());
	}

	if (SelectedSplineComponent.IsValid())
	{
		SelectedSplineComponent->Modify();
		if (AActor* Owner = SelectedSplineComponent->GetOwner())
		{
			Owner->Modify();
		}

		SelectedSplineComponent->SetSpline(CachedSplineCurves);
		CachedSplineCurves = FSplineCurves();

		SelectedSplineComponent->UpdateSpline();
		SelectedSplineComponent->bSplineHasBeenEdited = true;

		TArray<FProperty*> SplineProperties;
		for (const FName& Property : USplineComponent::GetSplinePropertyNames())
		{
			SplineProperties.Add(FindFProperty<FProperty>(USplineComponent::StaticClass(), Property));
		}
		
		FComponentVisualizer::NotifyPropertiesModified(SelectedSplineComponent.Get(), SplineProperties);
		
		GEditor->RedrawLevelEditingViewports(true);
	}
}

void USplineGeneratorBase::PreviewCurve()
{
	if (!SelectedSplineComponent.IsValid())
	{
		TSharedPtr<FSplineComponentVisualizer> SplineVis = WeakSplineVis.Pin();
		check(SplineVis);

		SelectedSplineComponent = MakeWeakObjectPtr(SplineVis->GetEditedSplineComponent());
	}

	if (SelectedSplineComponent.IsValid())
	{
		SelectedSplineComponent->Modify();
		if (AActor* Owner = SelectedSplineComponent->GetOwner())
		{
			Owner->Modify();
		}

		BuildCurve();

		SelectedSplineComponent->UpdateSpline();
		SelectedSplineComponent->bSplineHasBeenEdited = true;

		TArray<FProperty*> SplineProperties;
		for (const FName& Property : USplineComponent::GetSplinePropertyNames())
		{
			SplineProperties.Add(FindFProperty<FProperty>(USplineComponent::StaticClass(), Property));
		}
		
		FComponentVisualizer::NotifyPropertiesModified(SelectedSplineComponent.Get(), SplineProperties);

		GEditor->RedrawLevelEditingViewports(true);
	}
}

int32 USplineGeneratorBase::GetAddIndex(int32 Index) const
{
	switch (ShapeAddMode)
	{
	case EShapeAddMode::AppendAfter:
		return SelectedSplineComponent->GetNumberOfSplinePoints();
	case EShapeAddMode::AppendBefore:
		return 0;
	case EShapeAddMode::InsertAfter:
		return StartKey + Index + 1;
	case EShapeAddMode::InsertBefore:
		return StartKey;
	}

	return 0;
}

int32 USplineGeneratorBase::GetItrIndex(int32 Index) const
{
	switch (ShapeAddMode)
	{
	case EShapeAddMode::AppendAfter:
		return SelectedSplineComponent->GetNumberOfSplinePoints() - GetNumPoints() + Index;
	case EShapeAddMode::AppendBefore:
		return GetNumPoints() - Index - 1;
	case EShapeAddMode::InsertAfter:
		return StartKey + Index + 1;
	case EShapeAddMode::InsertBefore:
		return StartKey + GetNumPoints() - Index - 1;
	}

	return 0;
}

/*
 * Fancy math to recalculate tangents to turn into circle
 */
double CalcTangentMultiplier(const float InRadius, const float InRotInc)
{
	static constexpr double A = .5f;
	static constexpr double A2 = A * A;
	static constexpr double A3 = A2 * A;

	// Use first and second keys added as a sample calculation
	const FVector T0 = FVector::ForwardVector;
	const FVector T1 = T0.RotateAngleAxis(InRotInc, FVector::UpVector);
	const FVector P0 = FVector::RightVector * InRadius;
	const FVector P1 = P0.RotateAngleAxis(InRotInc, FVector::UpVector);

	// Calculate the difference between the actual interpolated midpoint and expected interpolated midpoint
	const FVector ActualVal = FMath::CubicInterp(P0, T0, P1, T1, A);
	const FVector ExpectedVal = P0.RotateAngleAxis(InRotInc * A, FVector::UpVector);
	const double Diff = (ActualVal.X - ExpectedVal.X);

	// Do a partial calculation of the cubic interpolation equation
	static constexpr double C1 = (A3 - (2 * A2) + A), C2 = (A3 - A2);
	const double PartialInterp = -1.f * ((C1 * T0.X) + (C2 * T1.X));

	// Calculate the final multiplier to multiply to all normalized tangents
	return FMath::IsNearlyZero(PartialInterp) ? 1.f : ((Diff / PartialInterp) + 1.f);
}

void UCircleSplineGenerator::BuildCurve()
{
	// Re-set the cached spline keys so we can add to them
	SelectedSplineComponent->SetSpline(CachedSplineCurves);

	// naming is a little weird, will fix
	const float BranchRightFlip = ((bKeepFirstKeyTangent && bBranchRight) ? 1.f : -1.f);
	const float ReverseDirFlip = ((bReverseDir) ^ !(ShapeAddMode & (EShapeAddMode::AppendBefore | EShapeAddMode::InsertBefore)) ? -1.f : 1.f);

	// Find starting features
	const FVector StartPoint = SelectedSplineComponent->GetLocationAtSplinePoint(GetAddIndex(-1), ESplineCoordinateSpace::Local);
	const FQuat StartQuat = SelectedSplineComponent->GetQuaternionAtSplinePoint(GetAddIndex(-1), ESplineCoordinateSpace::Local);

	const FVector CenterDir = bKeepFirstKeyTangent ? (StartQuat.GetRightVector() * BranchRightFlip) : StartQuat.GetForwardVector();
	const FVector CircleCenter = StartPoint + Radius * CenterDir;
	const float RotIncrement = (360.f / float(NumberOfPoints)) * ReverseDirFlip * BranchRightFlip * -1.f;

	// Add points around circle
	FVector CircleItr = StartPoint;
	for (int32 Index = 0; Index < NumberOfPoints; Index++)
	{
		const FVector Diff = CircleItr - CircleCenter;
		CircleItr = CircleCenter + Diff.RotateAngleAxis(RotIncrement, StartQuat.GetUpVector());

		SelectedSplineComponent->AddSplinePointAtIndex(CircleItr, GetAddIndex(Index), ESplineCoordinateSpace::Local);
	}

	float StartTangentFlip = BranchRightFlip * (bReverseDir ? 1.f : -1.f);
	const FVector StartCircleTangent = (bKeepFirstKeyTangent ? StartQuat.GetForwardVector() : StartQuat.GetRightVector()) * StartTangentFlip;

	if (NumberOfPoints >= 2)
	{
		const double TangentMult = CalcTangentMultiplier(Radius, RotIncrement) * ReverseDirFlip * -1.f;
		for (int32 Index = NumberOfPoints - 1; Index >= -1; Index--)
		{
			const FVector Tangent = StartCircleTangent.RotateAngleAxis(RotIncrement * float(Index + 1), StartQuat.GetUpVector()).GetSafeNormal();
			SelectedSplineComponent->SetRotationAtSplinePoint(GetItrIndex(Index), (FQuat::MakeFromEuler(FVector(0.f, 0.f, RotIncrement * float(Index + 1))) * StartQuat).Rotator(), ESplineCoordinateSpace::Local, false);
			SelectedSplineComponent->SetTangentAtSplinePoint(GetItrIndex(Index), Tangent * TangentMult, ESplineCoordinateSpace::Local);
		}
	}
}

void UArcSplineGenerator::BuildCurve()
{
	// Re-set the cached spline keys so we can add to them
	SelectedSplineComponent->SetSpline(CachedSplineCurves);

	// Prevent the curve from going whack
	Degrees = FMath::Clamp(Degrees, 0.f, 180.f * float(NumberOfPoints));

	const float BranchRightFlip = ((bKeepFirstKeyTangent && bBranchRight) ? 1.f : -1.f);
	const float ReverseDirFlip = ((bReverseDir) ^ !(ShapeAddMode & (EShapeAddMode::AppendBefore | EShapeAddMode::InsertBefore)) ? -1.f : 1.f) * BranchRightFlip * -1.f;
	// Find starting features
	const FVector StartPoint = SelectedSplineComponent->GetLocationAtSplinePoint(GetAddIndex(-1), ESplineCoordinateSpace::Local);
	const FQuat StartQuat = SelectedSplineComponent->GetQuaternionAtSplinePoint(GetAddIndex(-1), ESplineCoordinateSpace::Local);

	const FVector CenterDir = bKeepFirstKeyTangent ? StartQuat.GetRightVector() * BranchRightFlip : StartQuat.GetForwardVector();
	const FVector CircleCenter = StartPoint + Radius * CenterDir;
	const float RotIncrement = (Degrees / float(NumberOfPoints)) * ReverseDirFlip;

	// Add points around arc
	FVector CircleItr = StartPoint;
	for (int32 Index = 0; Index < NumberOfPoints; Index++)
	{
		const FVector Diff = CircleItr - CircleCenter;
		CircleItr = CircleCenter + Diff.RotateAngleAxis(RotIncrement, StartQuat.GetUpVector());

		SelectedSplineComponent->AddSplinePointAtIndex(CircleItr, GetAddIndex(Index), ESplineCoordinateSpace::Local);
	}

	float StartTangentFlip = (bReverseDir ? -1.f : 1.f);
 	const FVector StartCircleTangent = (bKeepFirstKeyTangent ? StartQuat.GetForwardVector() : StartQuat.GetRightVector()) * StartTangentFlip;

	if (NumberOfPoints >= 2)
	{
		const double TangentMult = CalcTangentMultiplier(Radius, RotIncrement) * ReverseDirFlip * -1.f;

		SelectedSplineComponent->SetTangentAtSplinePoint(GetItrIndex(-1), StartCircleTangent * TangentMult, ESplineCoordinateSpace::Local);
		for (int32 Index = 0; Index < NumberOfPoints; Index++)
		{
			const FVector Tangent = StartCircleTangent.RotateAngleAxis(RotIncrement * float(Index + 1), StartQuat.GetUpVector()).GetSafeNormal();
			SelectedSplineComponent->SetRotationAtSplinePoint(GetItrIndex(Index), (FQuat::MakeFromEuler(FVector(0.f, 0.f, RotIncrement * float(Index + 1))) * StartQuat).Rotator(), ESplineCoordinateSpace::Local, false);
			SelectedSplineComponent->SetTangentAtSplinePoint(GetItrIndex(Index), Tangent * TangentMult, ESplineCoordinateSpace::Local);
		}
	}
}

void UEllipseSplineGenerator::BuildCurve()
{
	// Re-set the cached spline keys so we can add to them
	SelectedSplineComponent->SetSpline(CachedSplineCurves);

	const float BranchRightFlip = ((bKeepFirstKeyTangent && bBranchRight) ? 1.f : -1.f);
	const float ReverseDirFlip = (bReverseDir ^ !(ShapeAddMode & (EShapeAddMode::AppendBefore | EShapeAddMode::InsertBefore)) ? -1.f : 1.f) * BranchRightFlip * -1.f;

	// Find starting features
	const FVector StartPoint = SelectedSplineComponent->GetLocationAtSplinePoint(GetAddIndex(-1), ESplineCoordinateSpace::Local);
	const FQuat StartQuat = SelectedSplineComponent->GetQuaternionAtSplinePoint(GetAddIndex(-1), ESplineCoordinateSpace::Local);

	const FVector CenterDir = bKeepFirstKeyTangent ? StartQuat.GetRightVector() * BranchRightFlip : StartQuat.GetForwardVector();
	const FVector CircleCenter = StartPoint + (bKeepFirstKeyTangent ? Length * .5f : Width * .5f) * CenterDir;

	// Add points around ellipse
	float CycleInc = (2 * PI / float(NumberOfPoints)) * ReverseDirFlip;
	float CycleItr = CycleInc + (bKeepFirstKeyTangent ? PI * .5f : PI) * BranchRightFlip * -1.f;
	const float TangentFlip = ReverseDirFlip * (!(ShapeAddMode & (EShapeAddMode::AppendBefore | EShapeAddMode::InsertBefore)) ? -1.f : 1.f) * -1.f;

	const FVector StartCircleTangent = (bKeepFirstKeyTangent ? StartQuat.GetForwardVector() : StartQuat.GetRightVector()) * (bReverseDir ? -1.f : 1.f);
	const FVector PrevStartTangent = SelectedSplineComponent->GetTangentAtSplinePoint(GetItrIndex(-1), ESplineCoordinateSpace::Local);
	SelectedSplineComponent->SetTangentAtSplinePoint(GetItrIndex(-1), StartCircleTangent * PrevStartTangent.Size(), ESplineCoordinateSpace::Local);
	for (int32 Index = 0; Index < NumberOfPoints; Index++, CycleItr += CycleInc)
	{
		const FVector EllipsePoint = CircleCenter + FVector(Width * .5f * FMath::Cos(CycleItr), Length * .5f * FMath::Sin(CycleItr), 0.f);
		const FVector EllipseTangent = FVector(Width * .5f * -1.f * FMath::Sin(CycleItr), Length * .5f * FMath::Cos(CycleItr), 0.f).GetSafeNormal();
		const FVector RotatedEllipsePoint = CircleCenter + StartQuat.Rotator().RotateVector(EllipsePoint - CircleCenter);
		const FVector RotatedEllipseTangent = StartQuat.Rotator().RotateVector(EllipseTangent) * TangentFlip;
		const int32 AddIdx = GetAddIndex(Index);
		SelectedSplineComponent->AddSplinePointAtIndex(RotatedEllipsePoint, AddIdx, ESplineCoordinateSpace::Local);
		const FVector PrevTangent = SelectedSplineComponent->GetTangentAtSplinePoint(AddIdx, ESplineCoordinateSpace::Local);
		SelectedSplineComponent->SetRotationAtSplinePoint(AddIdx, (FQuat::MakeFromEuler(FVector(0.f, 0.f, CycleItr)) * StartQuat).Rotator(), ESplineCoordinateSpace::Local, false);
		SelectedSplineComponent->SetTangentAtSplinePoint(AddIdx, RotatedEllipseTangent * PrevTangent.Size(), ESplineCoordinateSpace::Local);
	}
}

void USquareSplineGenerator::BuildCurve()
{
	// Re-set the cached spline keys so we can add to them
	SelectedSplineComponent->SetSpline(CachedSplineCurves);

	const bool bPrepend = !(ShapeAddMode & (EShapeAddMode::AppendAfter | EShapeAddMode::InsertAfter));
	const float LengthFlip = bPrepend ? -1.f : 1.f;

	// Find starting features
	const FVector StartPoint = SelectedSplineComponent->GetLocationAtSplinePoint(GetAddIndex(-1), ESplineCoordinateSpace::Local);
	const FQuat StartQuat = SelectedSplineComponent->GetQuaternionAtSplinePoint(GetAddIndex(-1), ESplineCoordinateSpace::Local);

	const float RotDir = bBranchRight ? 90.f : -90.f;

	FVector RectItr = StartPoint;
	FVector DirItr = StartQuat.GetForwardVector();

	for (int32 Index = 0; Index < 4; Index++)
	{
		RectItr += DirItr * Length * LengthFlip;
		const int32 AddIdx = GetAddIndex(Index);
		SelectedSplineComponent->AddSplinePointAtIndex(RectItr, AddIdx, ESplineCoordinateSpace::Local);
		SelectedSplineComponent->SetSplinePointType(bPrepend ? AddIdx : AddIdx - 1, ConvertInterpCurveModeToSplinePointType(CIM_Linear));
		SelectedSplineComponent->SetRotationAtSplinePoint(AddIdx, (FQuat::MakeFromEuler(FVector(0.f, 0.f, float(Index) * RotDir)) * StartQuat).Rotator(), ESplineCoordinateSpace::Local, false);
		DirItr = DirItr.RotateAngleAxis(RotDir, StartQuat.GetUpVector());
	}
}

void URectangleSplineGenerator::BuildCurve()
{
	// Re-set the cached spline keys so we can add to them
	SelectedSplineComponent->SetSpline(CachedSplineCurves);

	const bool bPrepend = !(ShapeAddMode & (EShapeAddMode::AppendAfter | EShapeAddMode::InsertAfter));
	const float LengthFlip = bPrepend ? -1.f : 1.f;

	// Find starting features
	const FVector StartPoint = SelectedSplineComponent->GetLocationAtSplinePoint(GetAddIndex(-1), ESplineCoordinateSpace::Local);
	const FQuat StartQuat = SelectedSplineComponent->GetQuaternionAtSplinePoint(GetAddIndex(-1), ESplineCoordinateSpace::Local);

	const float RotDir = bBranchRight ? 90.f : -90.f;

	int32 Index = 0;
	FVector RectItr = StartPoint;
	FVector DirItr = StartQuat.GetForwardVector();

	{
		// Draw first segment
		const int32 AddIdx = GetAddIndex(Index);
		RectItr += DirItr * Length * LengthFlip;
		SelectedSplineComponent->AddSplinePointAtIndex(RectItr, AddIdx, ESplineCoordinateSpace::Local);
		SelectedSplineComponent->SetSplinePointType(bPrepend ? AddIdx : AddIdx - 1, ConvertInterpCurveModeToSplinePointType(CIM_Linear));
		SelectedSplineComponent->SetRotationAtSplinePoint(AddIdx, (FQuat::MakeFromEuler(FVector(0.f, 0.f, float(Index) * RotDir)) * StartQuat).Rotator(), ESplineCoordinateSpace::Local, false);
		Index++;
	}

	{
		// Draw second segment
		const int32 AddIdx = GetAddIndex(Index);
		DirItr = DirItr.RotateAngleAxis(RotDir, StartQuat.GetUpVector());
		RectItr += DirItr * Width * LengthFlip;
		SelectedSplineComponent->AddSplinePointAtIndex(RectItr, AddIdx, ESplineCoordinateSpace::Local);
		SelectedSplineComponent->SetSplinePointType(bPrepend ? AddIdx : AddIdx - 1, ConvertInterpCurveModeToSplinePointType(CIM_Linear));
		SelectedSplineComponent->SetRotationAtSplinePoint(AddIdx, (FQuat::MakeFromEuler(FVector(0.f, 0.f, float(Index) * RotDir)) * StartQuat).Rotator(), ESplineCoordinateSpace::Local, false);
		Index++;
	}

	{
		// Draw third segment
		const int32 AddIdx = GetAddIndex(Index);
		DirItr = DirItr.RotateAngleAxis(RotDir, StartQuat.GetUpVector());
		RectItr += DirItr * Length * LengthFlip;
		SelectedSplineComponent->AddSplinePointAtIndex(RectItr, AddIdx, ESplineCoordinateSpace::Local);
		SelectedSplineComponent->SetSplinePointType(bPrepend ? AddIdx : AddIdx - 1, ConvertInterpCurveModeToSplinePointType(CIM_Linear));
		SelectedSplineComponent->SetRotationAtSplinePoint(AddIdx, (FQuat::MakeFromEuler(FVector(0.f, 0.f, float(Index) * RotDir)) * StartQuat).Rotator(), ESplineCoordinateSpace::Local, false);
		Index++;
	}

	{
		// Draw fourth segment
		const int32 AddIdx = GetAddIndex(Index);
		DirItr = DirItr.RotateAngleAxis(RotDir, StartQuat.GetUpVector());
		RectItr += DirItr * Width * LengthFlip;
		SelectedSplineComponent->AddSplinePointAtIndex(RectItr, AddIdx, ESplineCoordinateSpace::Local);
		SelectedSplineComponent->SetSplinePointType(bPrepend ? AddIdx : AddIdx - 1, ConvertInterpCurveModeToSplinePointType(CIM_Linear));
		SelectedSplineComponent->SetRotationAtSplinePoint(AddIdx, (FQuat::MakeFromEuler(FVector(0.f, 0.f, float(Index) * RotDir)) * StartQuat).Rotator(), ESplineCoordinateSpace::Local, false);
		Index++;
	}
}

void ULineSplineGenerator::BuildCurve()
{
	// Re-set the cached spline keys so we can add to them
	SelectedSplineComponent->SetSpline(CachedSplineCurves);

	const bool bPrepend = !(ShapeAddMode & (EShapeAddMode::AppendAfter | EShapeAddMode::InsertAfter));
	const double LengthFlip = bPrepend ? -1.f : 1.f;

	// Find starting features
	const FVector StartPoint = SelectedSplineComponent->GetLocationAtSplinePoint(GetAddIndex(-1), ESplineCoordinateSpace::Local);
	const FQuat StartQuat = SelectedSplineComponent->GetQuaternionAtSplinePoint(GetAddIndex(-1), ESplineCoordinateSpace::Local);

	double LineInc = 0.0f;
	FVector LineDir = FVector::ZeroVector;
	
	bEnableUpToNextPoint = false;
	int32 NextIdx = 0;
	if (ShapeAddMode == EShapeAddMode::InsertAfter || ShapeAddMode == EShapeAddMode::InsertBefore)
	{
		NextIdx = !(ShapeAddMode & EShapeAddMode::InsertAfter) ? StartKey - 1 : StartKey + 1;
		if (NextIdx >= 0 && NextIdx < SelectedSplineComponent->GetNumberOfSplinePoints())
		{
			bEnableUpToNextPoint = true;
		}
	}

	if (bUpToNextPoint && bEnableUpToNextPoint)
	{
		const FVector NextPoint = SelectedSplineComponent->GetLocationAtSplinePoint(NextIdx, ESplineCoordinateSpace::Local);
		const FVector Diff = NextPoint - StartPoint;
		Length = Diff.Size();
		LineDir = Diff.GetSafeNormal();
		LineInc = Diff.Size() / double(NumberOfPoints + 1);
	}
	else
	{
		LineInc = Length / double(NumberOfPoints);
		LineDir = StartQuat.GetForwardVector() * LengthFlip;
	}

	for (int32 Index = 0; Index < NumberOfPoints; Index++)
	{
		const int32 AddIdx = GetAddIndex(Index);
		const FVector NewPoint = StartPoint + (LineInc * double(Index + 1)) * LineDir;
		SelectedSplineComponent->AddSplinePointAtIndex(NewPoint, AddIdx, ESplineCoordinateSpace::Local);
	}
}

void AddGeneratorsToRegistry(TArray<UObject*>& InShapeGenRegistry)
{
	InShapeGenRegistry.Add(NewObject<UCircleSplineGenerator>());
	InShapeGenRegistry.Add(NewObject<UArcSplineGenerator>());
	InShapeGenRegistry.Add(NewObject<USquareSplineGenerator>());
	InShapeGenRegistry.Add(NewObject<UEllipseSplineGenerator>());
	InShapeGenRegistry.Add(NewObject<URectangleSplineGenerator>());
	InShapeGenRegistry.Add(NewObject<ULineSplineGenerator>());

	for (auto ShapeGen : InShapeGenRegistry)
	{
		ShapeGen->ClearFlags(RF_Transactional);
		ShapeGen->AddToRoot();
	}
}

void SSplineGeneratorPanel::Construct(const FArguments& InArgs, TWeakPtr<FSplineComponentVisualizer> InWeakSplineComponentVisualizer)
{
	WeakSplineComponentVisualizer = InWeakSplineComponentVisualizer;
	TSharedPtr<FSplineComponentVisualizer> SplineVis = WeakSplineComponentVisualizer.Pin();
	check(SplineVis);

	AddGeneratorsToRegistry(ShapeGenRegistry);

	FPropertyEditorModule& PropertyEditor = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");

	// Configure the Details View
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bUpdatesFromSelection = false;
	DetailsViewArgs.bLockable = false;
	DetailsViewArgs.ViewIdentifier = "SplineGenerationOptions";
	DetailsViewArgs.DefaultsOnlyVisibility = EEditDefaultsOnlyNodeVisibility::Automatic;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.bShowPropertyMatrixButton = false;
	DetailsViewArgs.bShowOptions = true;

	// Generate it and store a reference so we can update it with the right object later.
	DetailView = PropertyEditor.CreateDetailView(DetailsViewArgs);

	ActiveTransaction = MakeUnique<FScopedTransaction>(TEXT("SplineGenerator"), LOCTEXT("SplineGeneratorTransaction", "Generate Spline"), nullptr);

	TSharedRef<SVerticalBox> ShapeSelectorWidget = SNew(SVerticalBox);
	for (auto GenItr = ShapeGenRegistry.CreateConstIterator(); GenItr; ++GenItr)
	{
		auto OnRadioChanged = [this, GenObject = *GenItr] (ECheckBoxState CheckBoxState)
		{
			if (SplineGen)
			{
				SplineGen->Reset();
				SplineGen = nullptr;
				DetailView->SetObject(nullptr);
			}
			else
			{
				ActiveTransaction = MakeUnique<FScopedTransaction>(TEXT("SplineGenerator"), LOCTEXT("SplineGeneratorTransaction", "Generate Spline"), nullptr);
			}

			TSharedPtr<FSplineComponentVisualizer> SplineVis = WeakSplineComponentVisualizer.Pin();
			if (SplineVis.IsValid() && SplineVis->GetSelectedKeys().Num() == 1)
			{
				
				SplineGen = Cast<USplineGeneratorBase>(GenObject);
				check(SplineGen);
				SplineGen->Init(SplineVis);
				DetailView->SetObject(SplineGen);
			}
		};

		auto IsRadioChecked = [this, GenObject = *GenItr]()
		{
			if (SplineGen)
			{
				return GenObject == SplineGen ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			}
			return ECheckBoxState::Undetermined;
		};

		ShapeSelectorWidget->AddSlot()
		[
			SNew(SCheckBox)
			.Style(FAppStyle::Get(), "RadioButton")
			.IsChecked_Lambda(IsRadioChecked)
			.OnCheckStateChanged_Lambda(OnRadioChanged)
			.Padding(FMargin(10.f, 3.f))
			.Content()
			[
				SNew(STextBlock)
				.Text((*GenItr)->GetClass()->GetDisplayNameText())
				.Font(FCoreStyle::GetDefaultFontStyle("Regular", 10))
			]
		];
	}

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.HAlign(HAlign_Center)
		.AutoHeight()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("Shapes", "Shapes"))
			.Font(FCoreStyle::GetDefaultFontStyle("Regular", 11))
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBorder)
			.Padding(FMargin(5.f, 5.f))
			.BorderImage(FAppStyle::GetBrush("ToolPanel.DarkGroupBorder"))
			.Content()
			[
				ShapeSelectorWidget
			]
		]

		+ SVerticalBox::Slot()
		.FillHeight(1.f)
		[
			DetailView.ToSharedRef()
		]
	];
}

void SSplineGeneratorPanel::OnWindowClosed(const TSharedRef<SWindow>&)
{
	USplineComponent* SplineComp = WeakSplineComponentVisualizer.Pin()->GetEditedSplineComponent();

	if (SplineGen)
	{
		SplineGen = nullptr;
		DetailView->SetObject(nullptr);
	}

	ActiveTransaction.Reset();
}

void SSplineGeneratorPanel::OnSelectionUpdated()
{
	TSharedPtr<FSplineComponentVisualizer> SplineVis = WeakSplineComponentVisualizer.Pin();
	if (SplineGen && SplineVis.IsValid())
	{

		if (SplineVis->GetSelectedKeys().Num() > 0 && *SplineVis->GetSelectedKeys().CreateConstIterator() != SplineGen->StartKey)
		{
			ActiveTransaction.Reset();

			if (SplineVis->GetSelectedKeys().Num() == 1)
			{	
				ActiveTransaction  = MakeUnique<FScopedTransaction>(TEXT("SplineGenerator"), LOCTEXT("SplineGeneratorTransaction", "Generate Spline"), nullptr);
				SplineGen->Init(SplineVis);
			}
			else
			{
				SplineGen = nullptr;
				DetailView->SetObject(nullptr);
			}
		}
		else if (SplineVis->GetSelectedKeys().Num() == 0)
		{
			SplineGen = nullptr;
			DetailView->SetObject(nullptr);
			ActiveTransaction.Reset();
		}
	}
}

void SSplineGeneratorPanel::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (SplineGen)
	{
		SplineGen->PreviewCurve();
	}
}

void SSplineGeneratorPanel::OnFocusChanging(const FWeakWidgetPath& PreviousFocusPath, const FWidgetPath& NewWidgetPath, const FFocusEvent& InFocusEvent)
{
	FWidgetPath WidgetPath = NewWidgetPath.GetPathDownTo(SharedThis(this));
	FWidgetPath WidgetPath2 = NewWidgetPath;
	bool bPathFound = WidgetPath2.IsValid() && WidgetPath2.ExtendPathTo(FWidgetMatcher(SharedThis(this)));
	if (SplineGen && !WidgetPath.IsValid() && bPathFound)
	{
		SplineGen = nullptr;
		DetailView->SetObject(nullptr);
		ActiveTransaction.Reset();
	}
}

SSplineGeneratorPanel::~SSplineGeneratorPanel()
{
	if (ActiveTransaction.IsValid() && ActiveTransaction->IsOutstanding())
	{
		ActiveTransaction ->Cancel();
	}
}


#undef LOCTEXT_NAMESPACE
