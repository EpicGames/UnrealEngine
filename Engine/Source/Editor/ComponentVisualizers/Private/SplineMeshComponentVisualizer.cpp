// Copyright Epic Games, Inc. All Rights Reserved.

#include "SplineMeshComponentVisualizer.h"

#include "Components/ActorComponent.h"
#include "Components/SplineMeshComponent.h"
#include "Containers/Array.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "EditorViewportClient.h"
#include "Engine/EngineTypes.h"
#include "GameFramework/Actor.h"
#include "HAL/PlatformCrt.h"
#include "Math/Color.h"
#include "Math/InterpCurvePoint.h"
#include "Math/QuatRotationTranslationMatrix.h"
#include "Math/Transform.h"
#include "Math/Vector.h"
#include "Misc/AssertionMacros.h"
#include "PrimitiveDrawingUtils.h"
#include "Templates/Casts.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "UnrealWidgetFwd.h"
#include "Widgets/SNullWidget.h"

IMPLEMENT_HIT_PROXY(HSplineMeshVisProxy, HComponentVisProxy);
IMPLEMENT_HIT_PROXY(HSplineMeshKeyProxy, HSplineMeshVisProxy);
IMPLEMENT_HIT_PROXY(HSplineMeshTangentHandleProxy, HSplineMeshVisProxy);

#define LOCTEXT_NAMESPACE "SplineMeshComponentVisualizer"


FSplineMeshComponentVisualizer::FSplineMeshComponentVisualizer()
	: FComponentVisualizer()
	, SelectedKey(INDEX_NONE)
	, SelectedTangentHandle(INDEX_NONE)
	, SelectedTangentHandleType(ESelectedTangentHandle::None)
{
}

void FSplineMeshComponentVisualizer::OnRegister()
{
}

FSplineMeshComponentVisualizer::~FSplineMeshComponentVisualizer()
{
}

void FSplineMeshComponentVisualizer::DrawVisualization(const UActorComponent* Component, const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	if (const USplineMeshComponent* SplineMeshComp = Cast<const USplineMeshComponent>(Component))
	{
		if (!SplineMeshComp->bAllowSplineEditingPerInstance)
		{
			return;
		}

		USplineMeshComponent* EditedSplineMeshComp = GetEditedSplineMeshComponent();

		const FColor NormalColor = FColor::Red;
		const FColor SelectedColor = FColor::White;
		const FColor Color = (SplineMeshComp == EditedSplineMeshComp) ? SelectedColor : NormalColor;
		const float GrabHandleSize = 12.0f;
		const float TangentHandleSize = 10.0f;

		// Create a spline object
		FInterpCurveVector Spline = GetSpline(SplineMeshComp);

		// Draw the tangent handles before anything else so they will not overdraw the rest of the spline
		for (int32 PointIndex = 0; PointIndex < 2; PointIndex++)
		{
			const FVector KeyPos = SplineMeshComp->GetComponentTransform().TransformPosition(Spline.Points[PointIndex].OutVal);
			const FVector TangentWorldDirection = SplineMeshComp->GetComponentTransform().TransformVector(Spline.Points[PointIndex].LeaveTangent);

			PDI->SetHitProxy(NULL);
			const float DefaultDashSize = 5.f;
			const int32 MaxDashDrawCount = 65536;
			const double LineSize = TangentWorldDirection.Size();
			int64 DrawCount = FMath::CeilToInt64(LineSize / (2.f * DefaultDashSize));
			double DashSize = (DrawCount > MaxDashDrawCount) ? LineSize / (2.f * MaxDashDrawCount) : DefaultDashSize;
			DrawDashedLine(PDI, KeyPos, KeyPos + TangentWorldDirection, Color, DashSize, SDPG_Foreground);
			DrawDashedLine(PDI, KeyPos, KeyPos - TangentWorldDirection, Color, DashSize, SDPG_Foreground);
			PDI->SetHitProxy(new HSplineMeshTangentHandleProxy(Component, PointIndex, false));
			PDI->DrawPoint(KeyPos + TangentWorldDirection, Color, TangentHandleSize, SDPG_Foreground);
			PDI->SetHitProxy(new HSplineMeshTangentHandleProxy(Component, PointIndex, true));
			PDI->DrawPoint(KeyPos - TangentWorldDirection, Color, TangentHandleSize, SDPG_Foreground);
			PDI->SetHitProxy(NULL);
		}

		// Draw the keypoints
		for (int32 PointIndex = 0; PointIndex < 2; PointIndex++)
		{
			const FVector NewKeyPos = SplineMeshComp->GetComponentTransform().TransformPosition(Spline.Points[PointIndex].OutVal);
			PDI->SetHitProxy(new HSplineMeshKeyProxy(Component, PointIndex));
			PDI->DrawPoint(NewKeyPos, Color, GrabHandleSize, SDPG_Foreground);
			PDI->SetHitProxy(NULL);
		}

		// Draw the spline
		FVector StartPos = SplineMeshComp->GetComponentTransform().TransformPosition(Spline.Points[0].OutVal);
		for (int32 Step = 1; Step < 32; Step++)
		{
			const FVector EndPos = SplineMeshComp->GetComponentTransform().TransformPosition(Spline.Eval(static_cast<float>(Step) / 32.0f, FVector::ZeroVector));
			PDI->DrawLine(StartPos, EndPos, Color, SDPG_Foreground);
			StartPos = EndPos;
		}
	}
}

FInterpCurveVector FSplineMeshComponentVisualizer::GetSpline(const USplineMeshComponent* SplineMeshComp) const
{
	const FVector StartPosition = SplineMeshComp->GetStartPosition();
	const FVector EndPosition = SplineMeshComp->GetEndPosition();
	const FVector StartTangent = SplineMeshComp->GetStartTangent();
	const FVector EndTangent = SplineMeshComp->GetEndTangent();

	FInterpCurveVector Spline;
	Spline.Points.Reserve(2);
	Spline.Points.Emplace(0.0f, StartPosition, StartTangent, StartTangent, CIM_CurveUser);
	Spline.Points.Emplace(1.0f, EndPosition, EndTangent, EndTangent, CIM_CurveUser);

	return Spline;
}


bool FSplineMeshComponentVisualizer::VisProxyHandleClick(FEditorViewportClient* InViewportClient, HComponentVisProxy* VisProxy, const FViewportClick& Click)
{
	bool bEditing = false;

	if (VisProxy && VisProxy->Component.IsValid())
	{
		const USplineMeshComponent* SplineMeshComp = CastChecked<const USplineMeshComponent>(VisProxy->Component.Get());

		AActor* OldSplineMeshOwningActor = SplineMeshPropertyPath.GetParentOwningActor();
		SplineMeshPropertyPath = FComponentPropertyPath(SplineMeshComp);
		AActor* NewSplineMeshOwningActor = SplineMeshPropertyPath.GetParentOwningActor();

		if (SplineMeshPropertyPath.IsValid())
		{
			if (OldSplineMeshOwningActor != NewSplineMeshOwningActor)
			{
				SelectedKey = INDEX_NONE;
				SelectedTangentHandle = INDEX_NONE;
				SelectedTangentHandleType = ESelectedTangentHandle::None;
			}

			if (VisProxy->IsA(HSplineMeshKeyProxy::StaticGetType()))
			{
				// Control point clicked

				HSplineMeshKeyProxy* KeyProxy = static_cast<HSplineMeshKeyProxy*>(VisProxy);

				SelectedKey = KeyProxy->KeyIndex;
				SelectedTangentHandle = INDEX_NONE;
				SelectedTangentHandleType = ESelectedTangentHandle::None;

				bEditing = true;
			}
			else if (VisProxy->IsA(HSplineMeshTangentHandleProxy::StaticGetType()))
			{
				// Tangent handle clicked

				HSplineMeshTangentHandleProxy* KeyProxy = static_cast<HSplineMeshTangentHandleProxy*>(VisProxy);

				SelectedKey = INDEX_NONE;
				SelectedTangentHandle = KeyProxy->KeyIndex;
				SelectedTangentHandleType = KeyProxy->bArriveTangent ? ESelectedTangentHandle::Arrive : ESelectedTangentHandle::Leave;

				bEditing = true;
			}
		}
		else
		{
			SplineMeshPropertyPath.Reset();
		}
	}

	return bEditing;
}

USplineMeshComponent* FSplineMeshComponentVisualizer::GetEditedSplineMeshComponent() const
{
	return Cast<USplineMeshComponent>(SplineMeshPropertyPath.GetComponent());
}


bool FSplineMeshComponentVisualizer::GetWidgetLocation(const FEditorViewportClient* ViewportClient, FVector& OutLocation) const
{
	USplineMeshComponent* SplineMeshComp = GetEditedSplineMeshComponent();
	if (SplineMeshComp != nullptr)
	{
		FInterpCurveVector Spline = GetSpline(SplineMeshComp);

		if (SelectedTangentHandle != INDEX_NONE)
		{
			// If tangent handle index is set, use that
			check(SelectedTangentHandle < 2);
			const auto& Point = Spline.Points[SelectedTangentHandle];

			check(SelectedTangentHandleType != ESelectedTangentHandle::None);
			if (SelectedTangentHandleType == ESelectedTangentHandle::Leave)
			{
				OutLocation = SplineMeshComp->GetComponentTransform().TransformPosition(Point.OutVal + Point.LeaveTangent);
			}
			else if (SelectedTangentHandleType == ESelectedTangentHandle::Arrive)
			{
				OutLocation = SplineMeshComp->GetComponentTransform().TransformPosition(Point.OutVal - Point.ArriveTangent);
			}

			return true;
		}
		else if (SelectedKey != INDEX_NONE)
		{
			// Otherwise use the last key index set
			check(SelectedKey < 2);
			const auto& Point = Spline.Points[SelectedKey];
			OutLocation = SplineMeshComp->GetComponentTransform().TransformPosition(Point.OutVal);
			return true;
		}
	}

	return false;
}


bool FSplineMeshComponentVisualizer::GetCustomInputCoordinateSystem(const FEditorViewportClient* ViewportClient, FMatrix& OutMatrix) const
{
	if (ViewportClient->GetWidgetCoordSystemSpace() == COORD_Local)
	{
		USplineMeshComponent* SplineMeshComp = GetEditedSplineMeshComponent();
		if (SplineMeshComp != nullptr)
		{
			// First look at selected tangent handle for coordinate system
			int32 Index = SelectedTangentHandle;
			if (Index == INDEX_NONE)
			{
				// If not set, fall back to last key index selected
				Index = SelectedKey;
			}

			if (Index != INDEX_NONE)
			{
				FInterpCurveVector Spline = GetSpline(SplineMeshComp);

				check(Index < 2);
				check(SelectedTangentHandle != INDEX_NONE || SelectedKey != INDEX_NONE);

				const auto& Point = Spline.Points[Index];
				const FVector Tangent = Point.ArriveTangent.IsNearlyZero() ? FVector(1.0f, 0.0f, 0.0f) : Point.ArriveTangent.GetSafeNormal();
				const FVector Bitangent = (Tangent.Z == 1.0f) ? FVector(1.0f, 0.0f, 0.0f) : FVector(-Tangent.Y, Tangent.X, 0.0f).GetSafeNormal();
				const FVector Normal = FVector::CrossProduct(Tangent, Bitangent);

				OutMatrix = FMatrix(Tangent, Bitangent, Normal, FVector::ZeroVector) * FQuatRotationTranslationMatrix(SplineMeshComp->GetComponentTransform().GetRotation(), FVector::ZeroVector);
				return true;
			}
		}
	}

	return false;
}


bool FSplineMeshComponentVisualizer::HandleInputDelta(FEditorViewportClient* ViewportClient, FViewport* Viewport, FVector& DeltaTranslate, FRotator& DeltaRotate, FVector& DeltaScale)
{
	USplineMeshComponent* SplineMeshComp = GetEditedSplineMeshComponent();
	if (SplineMeshComp != nullptr)
	{
		if (SelectedTangentHandle != INDEX_NONE)
		{
			// When tangent handles are manipulated...

			check(SelectedTangentHandle < 2);
			const FVector OldTangent = (SelectedTangentHandle == 0) ? SplineMeshComp->GetStartTangent() : SplineMeshComp->GetEndTangent();

			if (!DeltaTranslate.IsZero())
			{
				check(SelectedTangentHandleType != ESelectedTangentHandle::None);

				const FVector Delta = (SelectedTangentHandleType == ESelectedTangentHandle::Leave) ? DeltaTranslate : -DeltaTranslate;
				const FVector NewTangent = OldTangent + SplineMeshComp->GetComponentTransform().InverseTransformVector(Delta);

				SplineMeshComp->Modify();

				if (SelectedTangentHandle == 0)
				{
					SplineMeshComp->SetStartTangent(NewTangent);
				}
				else
				{
					SplineMeshComp->SetEndTangent(NewTangent);
				}
			}
		}
		else
		{
			// When spline keys are manipulated...

			check(SelectedKey != INDEX_NONE);
			check(SelectedKey < 2);

			SplineMeshComp->Modify();

			FVector KeyPosition = (SelectedKey == 0) ? SplineMeshComp->GetStartPosition() : SplineMeshComp->GetEndPosition();
			FVector KeyTangent = (SelectedKey == 0) ? SplineMeshComp->GetStartTangent() : SplineMeshComp->GetEndTangent();

			bool bModifiedPosition = false;
			bool bModifiedTangent = false;

			if (!DeltaTranslate.IsZero())
			{
				// Find key position in world space
				const FVector CurrentWorldPos = SplineMeshComp->GetComponentTransform().TransformPosition(KeyPosition);
				// Move in world space
				const FVector NewWorldPos = CurrentWorldPos + DeltaTranslate;
				// Convert back to local space
				KeyPosition = SplineMeshComp->GetComponentTransform().InverseTransformPosition(NewWorldPos);

				bModifiedPosition = true;
			}

			if (!DeltaRotate.IsZero())
			{
				// Rotate tangent according to delta rotation
				KeyTangent = DeltaRotate.RotateVector(KeyTangent);

				bModifiedTangent = true;
			}

			if (!DeltaScale.IsZero())
			{
				// Break tangent into direction and length so we can change its scale (the 'tension')
				// independently of its direction.
				FVector Direction;
				double Length;
				KeyTangent.ToDirectionAndLength(Direction, Length);

				// Figure out which component has changed, and use it
				double DeltaScaleValue = (DeltaScale.X != 0.0f) ? DeltaScale.X : ((DeltaScale.Y != 0.0f) ? DeltaScale.Y : DeltaScale.Z);

				// Change scale, avoiding singularity by never allowing a scale of 0, hence preserving direction.
				Length += DeltaScaleValue * 10.0f;
				if (Length == 0.0f)
				{
					Length = SMALL_NUMBER;
				}

				KeyTangent = Direction * Length;

				bModifiedTangent = true;
			}

			if (bModifiedPosition)
			{
				if (SelectedKey == 0)
				{
					SplineMeshComp->SetStartPosition(KeyPosition);
				}
				else
				{
					SplineMeshComp->SetEndPosition(KeyPosition);
				}
			}

			if (bModifiedTangent)
			{
				if (SelectedKey == 0)
				{
					SplineMeshComp->SetStartTangent(KeyTangent);
				}
				else
				{
					SplineMeshComp->SetEndTangent(KeyTangent);
				}
			}
		}

		NotifyComponentModified();
		return true;
	}

	return false;
}

bool FSplineMeshComponentVisualizer::HandleInputKey(FEditorViewportClient* ViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event)
{
	return false;
}


void FSplineMeshComponentVisualizer::EndEditing()
{
	SplineMeshPropertyPath.Reset();
	SelectedKey = INDEX_NONE;
	SelectedTangentHandle = INDEX_NONE;
	SelectedTangentHandleType = ESelectedTangentHandle::None;
}


TSharedPtr<SWidget> FSplineMeshComponentVisualizer::GenerateContextMenu() const
{
	return SNullWidget::NullWidget;
}


void FSplineMeshComponentVisualizer::NotifyComponentModified()
{
	// Notify of change so any CS is re-run
	if (SplineMeshPropertyPath.IsValid())
	{
		SplineMeshPropertyPath.GetParentOwningActor()->PostEditMove(true);
	}

	GEditor->RedrawLevelEditingViewports(true);
}

#undef LOCTEXT_NAMESPACE
