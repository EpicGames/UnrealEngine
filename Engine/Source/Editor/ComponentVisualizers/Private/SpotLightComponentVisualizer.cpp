// Copyright Epic Games, Inc. All Rights Reserved.

#include "SpotLightComponentVisualizer.h"

#include "Components/ActorComponent.h"
#include "Components/SpotLightComponent.h"
#include "Engine/EngineTypes.h"
#include "Engine/TextureLightProfile.h"
#include "Math/Axis.h"
#include "Math/Color.h"
#include "Math/Transform.h"
#include "Math/UnrealMathSSE.h"
#include "PrimitiveDrawingUtils.h"
#include "SceneView.h"
#include "ShowFlags.h"
#include "Templates/Casts.h"
#include "UObject/ObjectPtr.h"



void FSpotLightComponentVisualizer::DrawVisualization( const UActorComponent* Component, const FSceneView* View, FPrimitiveDrawInterface* PDI )
{
	if(View->Family->EngineShowFlags.LightRadius)
	{
		const USpotLightComponent* SpotLightComp = Cast<const USpotLightComponent>(Component);
		if(SpotLightComp != NULL)
		{
			FTransform TransformNoScale = SpotLightComp->GetComponentTransform();
			TransformNoScale.RemoveScaling();

			// Draw point light source shape
			DrawWireCapsule(PDI, TransformNoScale.GetTranslation(), TransformNoScale.GetUnitAxis( EAxis::X ), TransformNoScale.GetUnitAxis( EAxis::Y ), TransformNoScale.GetUnitAxis( EAxis::Z ),
							FColor(231, 239, 0, 255), SpotLightComp->SourceRadius, 0.5f * SpotLightComp->SourceLength + SpotLightComp->SourceRadius, 25, SDPG_World);

			// Draw outer light cone
			DrawWireSphereCappedCone(PDI, TransformNoScale, SpotLightComp->AttenuationRadius, SpotLightComp->OuterConeAngle, 32, 8, 10, FColor(200, 255, 255), SDPG_World);

			// Draw inner light cone (if non zero)
			if(SpotLightComp->InnerConeAngle > KINDA_SMALL_NUMBER)
			{
				DrawWireSphereCappedCone(PDI, TransformNoScale, SpotLightComp->AttenuationRadius, SpotLightComp->InnerConeAngle, 32, 8, 10, FColor(150, 200, 255), SDPG_World);
			}

			if (SpotLightComp->IESTexture)
			{
				LightProfileVisualizer.DrawVisualization( SpotLightComp->IESTexture, TransformNoScale, View, PDI );
			}
		}
	}
}
