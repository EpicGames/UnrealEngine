// Copyright Epic Games, Inc. All Rights Reserved.

#include "SpringArmComponentVisualizer.h"

#include "Components/ActorComponent.h"
#include "Components/SceneComponent.h"
#include "Engine/EngineTypes.h"
#include "GameFramework/SpringArmComponent.h"
#include "HAL/Platform.h"
#include "Math/Color.h"
#include "Math/Transform.h"
#include "PrimitiveDrawingUtils.h"
#include "Templates/Casts.h"

static const FColor	ArmColor(255,0,0);

void FSpringArmComponentVisualizer::DrawVisualization(const UActorComponent* Component, const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	if (const USpringArmComponent* SpringArm = Cast<const USpringArmComponent>(Component))
	{
		PDI->DrawLine( SpringArm->GetComponentLocation(), SpringArm->GetSocketTransform(TEXT("SpringEndpoint"),RTS_World).GetTranslation(), ArmColor, SDPG_World );
	}
}
