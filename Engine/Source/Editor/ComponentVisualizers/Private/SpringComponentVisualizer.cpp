// Copyright Epic Games, Inc. All Rights Reserved.

#include "SpringComponentVisualizer.h"

#include "Components/ActorComponent.h"
#include "Engine/EngineTypes.h"
#include "Math/Axis.h"
#include "Math/Color.h"
#include "Math/Transform.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "PhysicsEngine/PhysicsSpringComponent.h"
#include "PrimitiveDrawingUtils.h"
#include "Templates/Casts.h"

static const FColor	DisabledColor(128, 128, 128);
static const FColor	CompressedColor(255, 0, 0);
static const FColor RestColor(0, 255, 0);

void FSpringComponentVisualizer::DrawVisualization( const UActorComponent* Component, const FSceneView* View, FPrimitiveDrawInterface* PDI )
{
	if (const UPhysicsSpringComponent* SpringComp = Cast<const UPhysicsSpringComponent>(Component))
	{
		//interpolate color by compression
		const float CompressionNormalized = SpringComp->GetNormalizedCompressionScalar();
		uint8 R = FMath::Lerp(RestColor.R, CompressedColor.R, CompressionNormalized);
		uint8 G = FMath::Lerp(RestColor.G, CompressedColor.G, CompressionNormalized);
		uint8 B = FMath::Lerp(RestColor.B, CompressedColor.B, CompressionNormalized);
		const FColor CurrentColor = SpringComp->IsActive() ? FColor(R, G, B) : DisabledColor;
		
		//draw capsule for spring sweep
		const FTransform& WorldTM = SpringComp->GetComponentToWorld();
		const FVector SpringStart = WorldTM.GetLocation();
		const FVector SpringEnd = SpringComp->GetSpringCurrentEndPoint();
		const double HalfHeight = (SpringEnd - SpringStart).Size() * 0.5f + SpringComp->SpringRadius;
		DrawWireCapsule(PDI, FMath::Lerp(SpringStart, SpringEnd, 0.5f), WorldTM.GetUnitAxis(EAxis::Z), WorldTM.GetUnitAxis(EAxis::Y), WorldTM.GetUnitAxis(EAxis::X), CurrentColor, SpringComp->SpringRadius, HalfHeight, 25, SDPG_World);
	}
}
