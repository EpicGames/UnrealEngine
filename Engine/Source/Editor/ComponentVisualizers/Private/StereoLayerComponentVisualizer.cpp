// Copyright Epic Games, Inc. All Rights Reserved.

#include "StereoLayerComponentVisualizer.h"

#include "Components/ActorComponent.h"
#include "Components/StereoLayerComponent.h"
#include "Templates/Casts.h"
#include "UObject/ObjectPtr.h"


void FStereoLayerComponentVisualizer::DrawVisualization( const UActorComponent* Component, const FSceneView* View, FPrimitiveDrawInterface* PDI )
{
	const UStereoLayerComponent* StereoLayerComp = Cast<const UStereoLayerComponent>(Component);
	if(StereoLayerComp != nullptr && StereoLayerComp->Shape != nullptr)
	{
		StereoLayerComp->Shape->DrawShapeVisualization(View, PDI);
    }
}
