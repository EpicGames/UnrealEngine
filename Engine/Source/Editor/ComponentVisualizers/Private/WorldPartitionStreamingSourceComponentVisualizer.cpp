// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartitionStreamingSourceComponentVisualizer.h"

#include "Components/ActorComponent.h"
#include "Components/WorldPartitionStreamingSourceComponent.h"
#include "Templates/Casts.h"

void FWorldPartitionStreamingSourceComponentVisualizer::DrawVisualization(const UActorComponent* Component, const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	if (const UWorldPartitionStreamingSourceComponent* StreamingSourceComponent = Cast<const UWorldPartitionStreamingSourceComponent>(Component))
	{
		StreamingSourceComponent->DrawVisualization(View, PDI);
	}
}
