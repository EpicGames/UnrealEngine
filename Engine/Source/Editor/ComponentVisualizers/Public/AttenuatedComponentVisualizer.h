// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/Attenuation.h"
#include "ComponentVisualizer.h"
#include "ShowFlags.h"
#include "SceneView.h"
#include "SceneManagement.h"

template<class T>
class TAttenuatedComponentVisualizer : public FComponentVisualizer
{
public:
	//~ Begin FComponentVisualizer Interface
	virtual void DrawVisualization(const UActorComponent* Component, const FSceneView* View, FPrimitiveDrawInterface* PDI) override
	{
		if (IsVisualizerEnabled(View->Family->EngineShowFlags))
		{
			const T* AttenuatedComponent = Cast<const T>(Component);
			if (AttenuatedComponent != nullptr)
			{
				const FTransform& Transform = AttenuatedComponent->GetComponentTransform();

				TMultiMap<EAttenuationShape::Type, FBaseAttenuationSettings::AttenuationShapeDetails> ShapeDetailsMap;
				AttenuatedComponent->CollectAttenuationShapesForVisualization(ShapeDetailsMap);

				const FVector Translation = Transform.GetTranslation();
				const FVector UnitXAxis   = Transform.GetUnitAxis( EAxis::X );
				const FVector UnitYAxis   = Transform.GetUnitAxis( EAxis::Y );
				const FVector UnitZAxis   = Transform.GetUnitAxis( EAxis::Z );

				for (const TPair<EAttenuationShape::Type, FBaseAttenuationSettings::AttenuationShapeDetails>& ShapeDetailsPair : ShapeDetailsMap)
				{
					const FColor OuterRadiusColor(255, 153, 0);
					const FColor InnerRadiusColor(216, 130, 0);

					const FBaseAttenuationSettings::AttenuationShapeDetails& ShapeDetails = ShapeDetailsPair.Value;
					switch (ShapeDetailsPair.Key)
					{
					case EAttenuationShape::Box:

						if (ShapeDetails.Falloff > 0.f)
						{
							DrawOrientedWireBox( PDI, Translation, UnitXAxis, UnitYAxis, UnitZAxis, ShapeDetails.Extents + FVector(ShapeDetails.Falloff), OuterRadiusColor, SDPG_World);
							DrawOrientedWireBox( PDI, Translation, UnitXAxis, UnitYAxis, UnitZAxis, ShapeDetails.Extents, InnerRadiusColor, SDPG_World);
						}
						else
						{
							DrawOrientedWireBox( PDI, Translation, UnitXAxis, UnitYAxis, UnitZAxis, ShapeDetails.Extents, OuterRadiusColor, SDPG_World);
						}
						break;

					case EAttenuationShape::Capsule:

						if (ShapeDetails.Falloff > 0.f)
						{
							DrawWireCapsule( PDI, Translation, UnitXAxis, UnitYAxis, UnitZAxis, OuterRadiusColor, ShapeDetails.Extents.Y + ShapeDetails.Falloff, ShapeDetails.Extents.X + ShapeDetails.Falloff, 25, SDPG_World);
							DrawWireCapsule( PDI, Translation, UnitXAxis, UnitYAxis, UnitZAxis, InnerRadiusColor, ShapeDetails.Extents.Y, ShapeDetails.Extents.X, 25, SDPG_World);
						}
						else
						{
							DrawWireCapsule( PDI, Translation, UnitXAxis, UnitYAxis, UnitZAxis, OuterRadiusColor, ShapeDetails.Extents.Y, ShapeDetails.Extents.X, 25, SDPG_World);
						}
						break;

					case EAttenuationShape::Cone:
					{
						FTransform Origin = Transform;
						Origin.SetScale3D(FVector(1.f));
						Origin.SetTranslation(Translation - (UnitXAxis * ShapeDetails.ConeOffset));

						if (ShapeDetails.Falloff > 0.f || ShapeDetails.Extents.Z > 0.f)
						{
							double ConeRadius = ShapeDetails.Extents.X + ShapeDetails.Falloff + ShapeDetails.ConeOffset;
							double ConeAngle = ShapeDetails.Extents.Y + ShapeDetails.Extents.Z;
							DrawWireSphereCappedCone(PDI, Origin, ConeRadius, ConeAngle, 16, 4, 10, OuterRadiusColor, SDPG_World);

							ConeRadius = ShapeDetails.Extents.X + ShapeDetails.ConeOffset;
							ConeAngle = ShapeDetails.Extents.Y;
							DrawWireSphereCappedCone(PDI, Origin, ConeRadius, ConeAngle, 16, 4, 10, InnerRadiusColor, SDPG_World );
						}
						else
						{
							const double ConeRadius = ShapeDetails.Extents.X + ShapeDetails.ConeOffset;
							const double ConeAngle = ShapeDetails.Extents.Y;
							DrawWireSphereCappedCone(PDI, Origin, ConeRadius, ConeAngle, 16, 4, 10, OuterRadiusColor, SDPG_World );
						}

						if (!FMath::IsNearlyZero(ShapeDetails.ConeSphereRadius, KINDA_SMALL_NUMBER))
						{
							if (ShapeDetails.ConeSphereFalloff > 0.f)
							{
								DrawWireSphereAutoSides(PDI, Origin, OuterRadiusColor, ShapeDetails.ConeSphereRadius + ShapeDetails.ConeSphereFalloff, SDPG_World);
								DrawWireSphereAutoSides(PDI, Origin, InnerRadiusColor, ShapeDetails.ConeSphereRadius, SDPG_World);
							}
							else
							{
								DrawWireSphereAutoSides(PDI, Origin, OuterRadiusColor, ShapeDetails.ConeSphereRadius, SDPG_World);
							}
						}
					}
					break;

					case EAttenuationShape::Sphere:

						if (ShapeDetails.Falloff > 0.f)
						{
							DrawWireSphereAutoSides(PDI, Translation, OuterRadiusColor, ShapeDetails.Extents.X + ShapeDetails.Falloff, SDPG_World);
							DrawWireSphereAutoSides(PDI, Translation, InnerRadiusColor, ShapeDetails.Extents.X, SDPG_World);
						}
						else
						{
							DrawWireSphereAutoSides(PDI, Translation, OuterRadiusColor, ShapeDetails.Extents.X, SDPG_World);
						}
						break;

					default:
						check(false);
					}
				}
			}
		}
	}
	//~ End FComponentVisualizer Interface

private:
	virtual bool IsVisualizerEnabled(const FEngineShowFlags& ShowFlags) const = 0;
};
