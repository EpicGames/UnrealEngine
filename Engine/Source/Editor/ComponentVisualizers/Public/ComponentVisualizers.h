// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class FComponentVisualizer;

class FComponentVisualizersModule : public IModuleInterface
{
public:
	/**
	 * Called right after the module DLL has been loaded and the module object has been created
	 */
	virtual void StartupModule() override;

	/**
	 * Called before the module is unloaded, right before the module object is destroyed.
	 */
	virtual void ShutdownModule() override;

	/**
	 * Override this to set whether your module is allowed to be unloaded on the fly
	 *
	 * @return	Whether the module supports shutdown separate from the rest of the engine.
	 */
	virtual bool SupportsDynamicReloading() override
	{
		return true;
	}

	/** Register a visualizer for a particular component class */
	COMPONENTVISUALIZERS_API void RegisterComponentVisualizer(FName ComponentClassName, TSharedPtr<FComponentVisualizer> Visualizer);

	/** Mutes a registered visualizer so that it will no longer be used. Can be restored without re-registering. */
	COMPONENTVISUALIZERS_API void MuteComponentVisualizer(FName ComponentClassName);

	/** Unmutes a muted visualizer so that it will be used again by the editor. */
	COMPONENTVISUALIZERS_API void UnmuteComponentVisualizer(FName ComponentClassName);

private:

	/** Array of component class names we have registered, so we know what to unregister afterwards */
	TArray<FName> RegisteredComponentClassNames;

	/** Set of muted component visualizers. We must retain the shared pointer so we can unregister from the editor. */
	TMap<FName, TSharedPtr<FComponentVisualizer>> MutedComponentVisualizers;
};
