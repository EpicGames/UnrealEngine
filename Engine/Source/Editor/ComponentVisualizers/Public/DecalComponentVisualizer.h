// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ComponentVisualizer.h"
#include "CoreMinimal.h"

#define UE_API COMPONENTVISUALIZERS_API

class FPrimitiveDrawInterface;
class FSceneView;
class UActorComponent;

class FDecalComponentVisualizer : public FComponentVisualizer
{
public:
	//~ Begin FComponentVisualizer Interface
	UE_API virtual void DrawVisualization(const UActorComponent* Component, const FSceneView* View, FPrimitiveDrawInterface* PDI) override;
	//~ End FComponentVisualizer Interface
};

#undef UE_API
