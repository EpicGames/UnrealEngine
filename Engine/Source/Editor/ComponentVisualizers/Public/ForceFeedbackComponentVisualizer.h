// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AttenuatedComponentVisualizer.h"
#include "Components/ForceFeedbackComponent.h"

class FForceFeedbackComponentVisualizer : public TAttenuatedComponentVisualizer<UForceFeedbackComponent>
{
private:
	virtual bool IsVisualizerEnabled(const FEngineShowFlags& ShowFlags) const override
	{
		return ShowFlags.ForceFeedbackRadius;
	}
};
