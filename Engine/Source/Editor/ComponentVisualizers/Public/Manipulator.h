// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "GameFramework/Actor.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "ViewportInteractableInterface.h"

#include "Manipulator.generated.h"


#define UE_API COMPONENTVISUALIZERS_API

PRAGMA_DISABLE_DEPRECATION_WARNINGS


class UActorComponent;
class UObject;
class USceneComponent;
class UViewportInteractor;
struct FHitResult;

UCLASS(MinimalAPI)
class UE_DEPRECATED(5.7, "The ViewportInteraction module is being deprecated alongside VR Editor mode.") AManipulator : public AActor, public IViewportInteractableInterface
{
	GENERATED_BODY()

public:

	UE_API AManipulator();

	// Begin AActor
	UE_API virtual void PostEditMove(bool bFinished) override;
	UE_API virtual bool IsEditorOnly() const final;
	// End AActor

	// Begin IViewportInteractableInterface
	UE_API virtual void OnPressed(UViewportInteractor* Interactor, const FHitResult& InHitResult, bool& bOutResultedInDrag) override;
	UE_API virtual void OnHover(UViewportInteractor* Interactor) override;
	UE_API virtual void OnHoverEnter(UViewportInteractor* Interactor, const FHitResult& InHitResult) override;
	UE_API virtual void OnHoverLeave(UViewportInteractor* Interactor, const UActorComponent* NewComponent) override;
	UE_API virtual void OnDragRelease(UViewportInteractor* Interactor) override;
	virtual class UViewportDragOperationComponent* GetDragOperationComponent() override { return nullptr; }
	virtual bool CanBeSelected() override { return true; };
	// End IViewportInteractableInterface

	/** Set the component that should be moved when the manipulator was moved. */
	UE_API void SetAssociatedComponent(USceneComponent* SceneComponent);

private:

	/** The component to transform when this manipulator was moved. */
	UPROPERTY()
	TObjectPtr<USceneComponent> AssociatedComponent;

	/** Visual representation of this manipulator. */
	UPROPERTY()
	TObjectPtr<class UStaticMeshComponent> StaticMeshComponent;

};

PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef UE_API
