// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ComponentVisualizer.h"
#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Math/Transform.h"

#define UE_API COMPONENTVISUALIZERS_API

class FPrimitiveDrawInterface;
class FSceneView;
class UActorComponent;
class UTextureLightProfile;

class FTextureLightProfileVisualizer
{
public:
	FTextureLightProfileVisualizer();

	void DrawVisualization(UTextureLightProfile* TextureLightProfile, const FTransform& LightTM, const FSceneView* View, FPrimitiveDrawInterface* PDI);

private:
	bool UpdateIntensitiesCache(UTextureLightProfile* TextureLightProfile, const FTransform& LightTM);

	const UTextureLightProfile* CachedLightProfile;
	TArray< float > IntensitiesCache;
};

class FPointLightComponentVisualizer : public FComponentVisualizer
{
public:
	//~ Begin FComponentVisualizer Interface
	UE_API virtual void DrawVisualization(const UActorComponent* Component, const FSceneView* View, FPrimitiveDrawInterface* PDI) override;
	//~ End FComponentVisualizer Interface

private:
	FTextureLightProfileVisualizer LightProfileVisualizer;
};

#undef UE_API
