// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ComponentVisualizer.h"
#include "Components/SplineComponent.h"
#include "Containers/Array.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Engine/EngineBaseTypes.h"
#include "GenericPlatform/ICursor.h"
#include "HitProxies.h"
#include "InputCoreTypes.h"
#include "Math/Axis.h"
#include "Math/Box.h"
#include "Math/InterpCurvePoint.h"
#include "Math/Matrix.h"
#include "Math/Quat.h"
#include "Math/Rotator.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "Misc/AssertionMacros.h"
#include "SplineDetailsProvider.h"
#include "Templates/SharedPointer.h"
#include "UObject/GCObject.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "Features/IModularFeature.h"
#include "PrimitiveDrawInterface.h"

#include "SplineComponentVisualizer.generated.h"

#define UE_API COMPONENTVISUALIZERS_API

class AActor;
class FCanvas;
class FEditorViewportClient;
class FMenuBuilder;
class FPrimitiveDrawInterface;
class FProperty;
class FSceneView;
class FUICommandList;
class FViewport;
class SSplineGeneratorPanel;
class SWidget;
class SWindow;
class UActorComponent;
class USplineComponent;
class USplineMetadata;
struct FConvexVolume;
struct FViewportClick;

namespace UE::SplineComponentVisualizer
{
	
struct FPDICache
{
	struct FRenderableLine
	{
		FRenderableLine()
			: Start(ForceInitToZero)
			, End(ForceInitToZero)
			, Color(ForceInitToZero)
			, DepthPriority(SDPG_World)
			, Thickness(0.0f)
			, DepthBias(0.0f)
		{}

		FRenderableLine(const FVector& InStart, const FVector& InEnd, const FColor& InColor, const ESceneDepthPriorityGroup InDepthPriority = SDPG_World, const float InThickness = 0.0f, const float InDepthBias = 0.0f)
			: Start(InStart)
			, End(InEnd)
			, Color(InColor)
			, DepthPriority(InDepthPriority)
			, Thickness(InThickness)
			, DepthBias(InDepthBias)
		{}

		void Draw(FPrimitiveDrawInterface* PDI) const
		{
			PDI->DrawLine(Start, End, Color, static_cast<uint8>(DepthPriority), Thickness, DepthBias);
		}

		FVector Start;
		FVector End;
		FColor Color;
		ESceneDepthPriorityGroup DepthPriority;
		float Thickness;
		float DepthBias;
	};
	
	struct FRenderablePoint
	{
		FRenderablePoint()
			: Position(ForceInitToZero)
			, Color(ForceInitToZero)
			, Size(0.f)
			, DepthPriority(SDPG_World)
		{}

		FRenderablePoint(const FVector& InPosition, const FColor& InColor, const float InSize, const ESceneDepthPriorityGroup InDepthPriority = SDPG_World)
			: Position(InPosition)
			, Color(InColor)
			, Size(InSize)
			, DepthPriority(InDepthPriority)
		{}

		void Draw(FPrimitiveDrawInterface* PDI) const
		{
			PDI->DrawPoint(Position, Color, Size, static_cast<uint8>(DepthPriority));
		}
		
		FVector Position;
		FColor Color;
		float Size;
		ESceneDepthPriorityGroup DepthPriority;
	};

	template <typename ElementType>
	struct TElementBatch
	{
		TArray<ElementType> Elements;
		TFunction<HHitProxy*()> AllocHitProxyFunc = []() -> HHitProxy* { return nullptr; };

		void Draw(FPrimitiveDrawInterface* PDI) const
		{
			PDI->SetHitProxy(AllocateHitProxy());
			for (const ElementType& Element : Elements) { Element.Draw(PDI); }
			PDI->SetHitProxy(nullptr);
		}
		
	private:
		
		HHitProxy* AllocateHitProxy() const
		{
			return AllocHitProxyFunc
				? AllocHitProxyFunc()
				: nullptr;
		}
	};
	
	using FLineBatch = TElementBatch<FRenderableLine>;
	using FPointBatch = TElementBatch<FRenderablePoint>;

	void Reset()
	{
		LineBatches.Reset();
		PointBatches.Reset();
	}
	
	void Draw(FPrimitiveDrawInterface* PDI) const
	{
		for (const FLineBatch& Batch : LineBatches) { Batch.Draw(PDI); }
		for (const FPointBatch& Batch : PointBatches) { Batch.Draw(PDI); }
	}
	
	void AddBatch(FLineBatch&& LineBatch)
	{
		LineBatches.Add(MoveTemp(LineBatch));
		LineBatch.Elements.Reset();
	}

	void AddBatch(FPointBatch&& PointBatch)
	{
		PointBatches.Add(MoveTemp(PointBatch));
		PointBatch.Elements.Reset();
	}
	
	TArray<FLineBatch> LineBatches;
	TArray<FPointBatch> PointBatches;
	bool bDirty = true;
};
	
}

/** Tangent handle selection modes. */
UENUM()
enum class ESelectedTangentHandle
{
	None,
	Leave,
	Arrive
};

/** Selection state data that will be captured by scoped transactions.*/
UCLASS(MinimalAPI, Transient)
class USplineComponentVisualizerSelectionState : public UObject
{
	GENERATED_BODY()

public:

	/** Checks LastKeyIndexSelected is valid given the number of splint points and returns its value. */
	UE_API int32 GetVerifiedLastKeyIndexSelected(const int32 InNumSplinePoints) const;

	/** Checks TangentHandle and TangentHandleType are valid and sets relevant output parameters. */
	UE_API void GetVerifiedSelectedTangentHandle(const int32 InNumSplinePoints, int32& OutSelectedTangentHandle, ESelectedTangentHandle& OutSelectedTangentHandleType) const;

	const FComponentPropertyPath GetSplinePropertyPath() const { return SplinePropertyPath; }
	void SetSplinePropertyPath(const FComponentPropertyPath& InSplinePropertyPath) { SplinePropertyPath = InSplinePropertyPath; }

	const TSet<int32>& GetSelectedKeys() const { return SelectedKeys; }
	TSet<int32>& ModifySelectedKeys() { return SelectedKeys; }

	int32 GetLastKeyIndexSelected() const { return LastKeyIndexSelected; }
	void SetLastKeyIndexSelected(const int32 InLastKeyIndexSelected) { LastKeyIndexSelected = InLastKeyIndexSelected; }

	int32 GetSelectedSegmentIndex() const { return SelectedSegmentIndex; }
	void SetSelectedSegmentIndex(const int32 InSelectedSegmentIndex) { SelectedSegmentIndex = InSelectedSegmentIndex; }

	int32 GetSelectedTangentHandle() const { return SelectedTangentHandle; }
	void SetSelectedTangentHandle(const int32 InSelectedTangentHandle) { SelectedTangentHandle = InSelectedTangentHandle; }

	int32 GetSelectedAttributeIndex() const { return SelectedAttributeIndex; }
	FName GetSelectedAttributeName() const { return SelectedAttributeName; }
	void SetSelectedAttribute(const int32 InSelectedAttributeIndex = INDEX_NONE, const FName& InSelectedAttributeName = NAME_None)
	{
		if (InSelectedAttributeIndex != INDEX_NONE)
		{
			// if we actually select an attribute point, don't persist other selection.
			ClearSelectedKeys();	// necessary? maybe...
		}
		SelectedAttributeIndex = InSelectedAttributeIndex;
		SelectedAttributeName = InSelectedAttributeName;
	}

	ESelectedTangentHandle GetSelectedTangentHandleType() const { return SelectedTangentHandleType; }
	void SetSelectedTangentHandleType(const ESelectedTangentHandle InSelectedTangentHandle) { SelectedTangentHandleType = InSelectedTangentHandle; }

	FVector GetSelectedSplinePosition() const { return SelectedSplinePosition; }
	void SetSelectedSplinePosition(const FVector& InSelectedSplinePosition) { SelectedSplinePosition = InSelectedSplinePosition; }

	FQuat GetCachedRotation() const { return CachedRotation; }
	void SetCachedRotation(const FQuat& InCachedRotation) { CachedRotation = InCachedRotation; }

	UE_API void Reset();
	UE_API void ClearSelectedKeys();
	UE_API void ClearSelectedSegmentIndex();
	UE_API void ClearSelectedTangentHandle();
	void ClearSelectedAttribute() { SetSelectedAttribute(); }

	UE_API bool IsSplinePointSelected(const int32 InIndex) const;

protected:
	/** Property path from the parent actor to the component */
	UPROPERTY()
	FComponentPropertyPath SplinePropertyPath;

	/** Indices of keys we have selected */
	UPROPERTY()
	TSet<int32> SelectedKeys;

	/** Index of the last key we selected */
	UPROPERTY()
	int32 LastKeyIndexSelected = INDEX_NONE;

	/** Index of segment we have selected */
	UPROPERTY()
	int32 SelectedSegmentIndex = INDEX_NONE;

	/** Index of tangent handle we have selected */
	UPROPERTY()
	int32 SelectedTangentHandle = INDEX_NONE;

	/** The type of the selected tangent handle */
	UPROPERTY()
	ESelectedTangentHandle SelectedTangentHandleType = ESelectedTangentHandle::None;

	/** Index of attribute handle we have selected */
	UPROPERTY()
	int32 SelectedAttributeIndex = INDEX_NONE;

	UPROPERTY()
	FName SelectedAttributeName = NAME_None;
	
	/** Position on spline we have selected */
	UPROPERTY()
	FVector SelectedSplinePosition;

	/** Cached rotation for this point */
	UPROPERTY()
	FQuat CachedRotation;
};

/** Base class for clickable spline editing proxies */
struct HSplineVisProxy : public HComponentVisProxy
{
	DECLARE_HIT_PROXY( UE_API );

	HSplineVisProxy(const UActorComponent* InComponent)
	: HComponentVisProxy(InComponent, HPP_Wireframe)
	{}

	virtual EMouseCursor::Type GetMouseCursor() override
	{
		return EMouseCursor::CardinalCross;
	}
};

/** Proxy for a spline key */
struct HSplineKeyProxy : public HSplineVisProxy
{
	DECLARE_HIT_PROXY( UE_API );

	HSplineKeyProxy(const UActorComponent* InComponent, int32 InKeyIndex) 
		: HSplineVisProxy(InComponent)
		, KeyIndex(InKeyIndex)
	{}

	int32 KeyIndex;

	virtual EMouseCursor::Type GetMouseCursor() override
	{
		return EMouseCursor::CardinalCross;
	}
};

/** Proxy for spline attribute key (as opposed to a positional control point) */
struct HSplineAttributeKeyProxy : public HSplineVisProxy
{
	DECLARE_HIT_PROXY( UE_API );

	HSplineAttributeKeyProxy(const UActorComponent* InComponent, int32 InKeyIndex) 
		: HSplineVisProxy(InComponent)
		, KeyIndex(InKeyIndex)
	{}

	int32 KeyIndex;
	
	virtual EMouseCursor::Type GetMouseCursor() override
	{
		return EMouseCursor::CardinalCross;
	}
};

/** Proxy for a spline segment */
struct HSplineSegmentProxy : public HSplineVisProxy
{
	DECLARE_HIT_PROXY( UE_API );

	HSplineSegmentProxy(const UActorComponent* InComponent, int32 InSegmentIndex)
		: HSplineVisProxy(InComponent)
		, SegmentIndex(InSegmentIndex)
	{}

	int32 SegmentIndex;

	virtual EMouseCursor::Type GetMouseCursor() override
	{
		return EMouseCursor::CardinalCross;
	}
};

/** Proxy for a tangent handle */
struct HSplineTangentHandleProxy : public HSplineVisProxy
{
	DECLARE_HIT_PROXY( UE_API );

	HSplineTangentHandleProxy(const UActorComponent* InComponent, int32 InKeyIndex, bool bInArriveTangent)
		: HSplineVisProxy(InComponent)
		, KeyIndex(InKeyIndex)
		, bArriveTangent(bInArriveTangent)
	{}

	int32 KeyIndex;
	bool bArriveTangent;

	virtual EMouseCursor::Type GetMouseCursor() override
	{
		return EMouseCursor::CardinalCross;
	}
};

/** Accepted modes for snapping points. */
enum class ESplineComponentSnapMode
{
	Snap,
	AlignToTangent,
	AlignPerpendicularToTangent
};

/** SplineComponent visualizer/edit functionality */
class FSplineComponentVisualizer : public FComponentVisualizer, public FGCObject
PRAGMA_DISABLE_INTERNAL_WARNINGS
	, public ISplineDetailsProvider
PRAGMA_DISABLE_INTERNAL_WARNINGS
{
	using FPDICache = UE::SplineComponentVisualizer::FPDICache;
	
public:

	UE_API FSplineComponentVisualizer();
	UE_API virtual ~FSplineComponentVisualizer();

	//~ Begin FComponentVisualizer Interface
	UE_API virtual void OnRegister() override;
	UE_API virtual bool ShouldShowForSelectedSubcomponents(const UActorComponent* Component) override;
	UE_API virtual void DrawVisualization(const UActorComponent* Component, const FSceneView* View, FPrimitiveDrawInterface* PDI) override;
	UE_API virtual bool VisProxyHandleClick(FEditorViewportClient* InViewportClient, HComponentVisProxy* VisProxy, const FViewportClick& Click) override;
	/** Draw HUD on viewport for the supplied component */
	UE_API virtual void DrawVisualizationHUD(const UActorComponent* Component, const FViewport* Viewport, const FSceneView* View, FCanvas* Canvas) override;
	UE_API virtual void EndEditing() override;
	UE_API virtual bool GetWidgetLocation(const FEditorViewportClient* ViewportClient, FVector& OutLocation) const override;
	UE_API virtual bool GetCustomInputCoordinateSystem(const FEditorViewportClient* ViewportClient, FMatrix& OutMatrix) const override;
	UE_API virtual bool HandleInputDelta(FEditorViewportClient* ViewportClient, FViewport* Viewport, FVector& DeltaTranslate, FRotator& DeltaRotate, FVector& DeltaScale) override;
	UE_API virtual bool HandleInputKey(FEditorViewportClient* ViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event) override;
	/** Handle click modified by Alt, Ctrl and/or Shift. The input HitProxy may not be on this component. */
	UE_API virtual bool HandleModifiedClick(FEditorViewportClient* InViewportClient, HHitProxy* HitProxy, const FViewportClick& Click) override;
	/** Handle box select input */
	UE_API virtual bool HandleBoxSelect(const FBox& InBox, FEditorViewportClient* InViewportClient, FViewport* InViewport) override;
	/** Handle frustum select input */
	UE_API virtual bool HandleFrustumSelect(const FConvexVolume& InFrustum, FEditorViewportClient* InViewportClient, FViewport* InViewport) override;
	/** Return whether focus on selection should focus on bounding box defined by active visualizer */
	UE_API virtual bool HasFocusOnSelectionBoundingBox(FBox& OutBoundingBox) override;
	/** Pass snap input to active visualizer */
	UE_API virtual bool HandleSnapTo(const bool bInAlign, const bool bInUseLineTrace, const bool bInUseBounds, const bool bInUsePivot, AActor* InDestination) override;
	/** Gets called when the mouse tracking has stopped (dragging behavior) */
	UE_API virtual void TrackingStopped(FEditorViewportClient* InViewportClient, bool bInDidMove) override;
	/** Get currently edited component, this is needed to reset the active visualizer after undo/redo */
	UE_API virtual UActorComponent* GetEditedComponent() const override;
	UE_API virtual TSharedPtr<SWidget> GenerateContextMenu() const override;
	UE_API virtual bool IsVisualizingArchetype() const override;
	//~ End FComponentVisualizer Interface

	/** Add menu sections to the context menu */
	UE_API virtual void GenerateContextMenuSections(FMenuBuilder& InMenuBuilder) const;

	/** Get the spline component we are currently editing */
	UE_API virtual USplineComponent* GetEditedSplineComponent() const override;

	const virtual TSet<int32>& GetSelectedKeys() const override { check(SelectionState); return SelectionState->GetSelectedKeys(); }

	/** Select first or last spline point, returns true if the spline component being edited has changed */
	UE_API virtual bool HandleSelectFirstLastSplinePoint(USplineComponent* InSplineComponent, bool bFirstPoint) override;

	UE_API virtual void HandleSelectPrevNextSplinePoint(bool bNext, bool bAddToSelection) override;

	/** Select all spline points, , returns true if the spline component being edited has changed */
	UE_API virtual bool HandleSelectAllSplinePoints(USplineComponent* InSplineComponent) override;

	/** Select next or prev spline point, loops when last point is currently selected */
	UE_API void OnSelectPrevNextSplinePoint(bool bNextPoint, bool bAddToSelection);

	/** Sets the new cached rotation on the visualizer */
	UE_API virtual void SetCachedRotation(const FQuat& NewRotation) override;
protected:

	/** Determine if any selected key index is out of range (perhaps because something external has modified the spline) */
	UE_API bool IsAnySelectedKeyIndexOutOfRange(const USplineComponent* Comp) const;

	/** Whether a single spline key is currently selected */
	UE_API bool IsSingleKeySelected() const;
	
	/** Whether a multiple spline keys are currently selected */
	UE_API bool AreMultipleKeysSelected() const;

	/** Whether any keys are currently selected */
	UE_API bool AreKeysSelected() const;

	/** Select spline point at specified index */
	UE_API void SelectSplinePoint(int32 SelectIndex, bool bAddToSelection);

	/** Transforms selected tangent by given translation */
	UE_API bool TransformSelectedTangent(EPropertyChangeType::Type InPropertyChangeType, const FVector& InDeltaTranslate);

	/** Transforms selected tangent by given translate, rotate and scale */
	UE_API bool TransformSelectedKeys(EPropertyChangeType::Type InPropertyChangeType, const FVector& InDeltaTranslate, const FRotator& InDeltaRotate = FRotator::ZeroRotator, const FVector& InDeltaScale = FVector::ZeroVector);

	/** Transforms selected attribute by given translation */
	UE_API bool TransformSelectedAttribute(EPropertyChangeType::Type InPropertyChangeType, const FVector& InDeltaTranslate);
	
	/** Update the key selection state of the visualizer */
	UE_API virtual void ChangeSelectionState(int32 Index, bool bIsCtrlHeld);

	/** Alt-drag: duplicates the selected spline key */
	UE_API virtual bool DuplicateKeyForAltDrag(const FVector& InDrag);

	/** Alt-drag: updates duplicated selected spline key */
	UE_API virtual bool UpdateDuplicateKeyForAltDrag(const FVector& InDrag);

	/** Return spline data for point on spline closest to input point */
	UE_API virtual float FindNearest(const FVector& InLocalPos, int32 InSegmentStartIndex, FVector& OutSplinePos, FVector& OutSplineTangent) const;

	/** Split segment using given world position */
	UE_API virtual void SplitSegment(const FVector& InWorldPos, int32 InSegmentIndex, bool bCopyFromSegmentBeginIndex = true);

	/** Update split segment based on drag offset */
	UE_API virtual void UpdateSplitSegment(const FVector& InDrag);

	/** Add segment to beginning or end of spline */
	UE_API virtual void AddSegment(const FVector& InWorldPos, bool bAppend);

	/** Add segment to beginning or end of spline */
	UE_API virtual void UpdateAddSegment(const FVector& InWorldPos);

	/** Alt-drag: duplicates the selected spline key */
	UE_API virtual void ResetAllowDuplication();

	/** Snapping: snap keys to axis position of last selected key */
	UE_API virtual void SnapKeysToLastSelectedAxisPosition(const EAxis::Type InAxis, TArray<int32> InSnapKeys);

	/** Snapping: snap key to selected actor */
	UE_API virtual void SnapKeyToActor(const AActor* InActor, const ESplineComponentSnapMode SnapMode);

	/** Snapping: generic method for snapping selected keys to given transform */
	UE_API virtual void SnapKeyToTransform(const ESplineComponentSnapMode InSnapMode,
		const FVector& InWorldPos,
		const FVector& InWorldUpVector,
		const FVector& InWorldForwardVector,
		const FVector& InScale,
		const USplineMetadata* InCopySplineMetadata = nullptr,
		const int32 InCopySplineMetadataKey = 0);

	/** Snapping: set snap to actor temporary mode */
	UE_API virtual void SetSnapToActorMode(const bool bInIsSnappingToActor, const ESplineComponentSnapMode InSnapMode = ESplineComponentSnapMode::Snap);

	/** Snapping: get snap to actor temporary mode */
	UE_API virtual bool GetSnapToActorMode(ESplineComponentSnapMode& OutSnapMode) const;

	/** Reset temporary modes after inputs are handled. */
	UE_API virtual void ResetTempModes();

	/** Updates the component and selected properties if the component has changed */
	UE_API const USplineComponent* UpdateSelectedSplineComponent(HComponentVisProxy* VisProxy);

	UE_API void OnDeleteKey();
	UE_API bool CanDeleteKey() const;

	/** Duplicates selected spline keys in place */
	UE_API void OnDuplicateKey();
	UE_API bool IsKeySelectionValid() const;

	UE_API void OnAddKeyToSegment();
	UE_API bool CanAddKeyToSegment() const;

	UE_API void OnSnapKeyToNearestSplinePoint(ESplineComponentSnapMode InSnapMode);

	UE_API void OnSnapKeyToActor(const ESplineComponentSnapMode InSnapMode);

	UE_API void OnSnapAllToAxis(EAxis::Type InAxis);

	UE_API void OnSnapSelectedToAxis(EAxis::Type InAxis);

	UE_API void OnStraightenKey(int32 Direction);
	UE_API void StraightenKey(int32 KeyToStraighten, int32 KeyToStraightenToward);

	UE_API void OnToggleSnapTangentAdjustment();
	UE_API bool IsSnapTangentAdjustment() const;

	UE_API void OnLockAxis(EAxis::Type InAxis);
	UE_API bool IsLockAxisSet(EAxis::Type InAxis) const; 

	UE_API void OnResetToAutomaticTangent(EInterpCurveMode Mode);
	UE_API bool CanResetToAutomaticTangent(EInterpCurveMode Mode) const;

	UE_API void OnSetKeyType(EInterpCurveMode Mode);
	UE_API bool IsKeyTypeSet(EInterpCurveMode Mode) const;

	UE_API void OnSetVisualizeRollAndScale();
	UE_API bool IsVisualizingRollAndScale() const;

	UE_API void OnSetDiscontinuousSpline();
	UE_API bool IsDiscontinuousSpline() const;

	UE_API void OnToggleClosedLoop();
	UE_API bool IsClosedLoop() const;

	UE_API void OnResetToDefault();
	UE_API bool CanResetToDefault() const;

	UE_API void OnAddAttributeKey();
	UE_API bool CanAddAttributeKey() const;

	UE_API void OnDeleteAttributeKey();
	UE_API bool CanDeleteAttributeKey() const;
	
	/** Select first or last spline point */
	UE_API void OnSelectFirstLastSplinePoint(bool bFirstPoint);

	/** Select all spline points, if no spline points selected yet the currently edited spline component will be set as well */
	UE_API void OnSelectAllSplinePoints();

	UE_API bool CanSelectSplinePoints() const;

	/** Generate the submenu containing available selection actions */
	UE_API void GenerateSelectSplinePointsSubMenu(FMenuBuilder& MenuBuilder) const;

	/** Generate the submenu containing the available point types */
	UE_API void GenerateSplinePointTypeSubMenu(FMenuBuilder& MenuBuilder) const;

	/** Generate the submenu containing the available auto tangent types */
	UE_API void GenerateTangentTypeSubMenu(FMenuBuilder& MenuBuilder) const;

	/** Generate the submenu containing the available snap/align actions */
	UE_API void GenerateSnapAlignSubMenu(FMenuBuilder& MenuBuilder) const;
	
	/** Generate the submenu containing the lock axis types */
	UE_API void GenerateLockAxisSubMenu(FMenuBuilder& MenuBuilder) const;

	/** Extends the provided MenuBuilder with an Attribute section */
	UE_API void GenerateAttributeMenuSection(FMenuBuilder& InMenuBuilder) const;
	
	/** Generate the channel creator/selector widget. */
	UE_API TSharedPtr<SWidget> GenerateChannelWidget() const;
	UE_API TSharedPtr<SWidget> GenerateChannelCreatorWidget() const;
	UE_API TSharedPtr<SWidget> GenerateChannelSelectorWidget() const;

	/** Generate the attribute editor widget. */
	UE_API TSharedPtr<SWidget> GenerateAttributeEditorWidget() const;

	/** Helper function to set edited component we are currently editing */
	UE_API void SetEditedSplineComponent(const USplineComponent* InSplineComponent);

	UE_API void CreateSplineGeneratorPanel();
	
	UE_API void OnDeselectedInEditor(TObjectPtr<USplineComponent> SplineComponent);

	UE_API bool ShouldUseForSpline(const USplineComponent* SplineComponent) const override;
	UE_API void ActivateVisualization() override;
	UE_API bool IsEnabledForSpline(const USplineComponent* InSplineComponent) const;
	
	// FGCObject interface
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector);
	virtual FString GetReferencerName() const override
	{
		return TEXT("FSplineComponentVisualizer");
	}
	// End of FGCObject interface

	/** Output log commands */
	TSharedPtr<FUICommandList> SplineComponentVisualizerActions;

	/** Current selection state */
	TObjectPtr<USplineComponentVisualizerSelectionState> SelectionState;

	/** Whether we currently allow duplication when dragging */
	bool bAllowDuplication;

	/** Alt-drag: True when in process of duplicating a spline key. */
	bool bDuplicatingSplineKey;

	/** Alt-drag: True when in process of adding end segment. */
	bool bUpdatingAddSegment;

	/** Alt-drag: Delays duplicating control point to accumulate sufficient drag input offset. */
	uint32 DuplicateDelay;

	/** Alt-drag: Accumulates delayed drag offset. */
	FVector DuplicateDelayAccumulatedDrag;

	/** Alt-drag: Cached segment parameter for split segment at new control point */
	float DuplicateCacheSplitSegmentParam;

	/** Axis to fix when adding new spline points. Uses the value of the currently 
	    selected spline point's X, Y, or Z value when fix is not equal to none. */
	EAxis::Type AddKeyLockedAxis;

	/** Snap: True when in process of snapping to actor which needs to be Ctrl-Selected. */
	bool bIsSnappingToActor;

	/** Snap: Snap to actor mode. */
	ESplineComponentSnapMode SnapToActorMode;

	UE_DEPRECATED(5.7, "SplineCurvesProperty is deprecated, please use SplineProperties.")
	FProperty* SplineCurvesProperty;

	/** The set of properties to mark as dirty when manipulating the target spline component. */
	TArray<FProperty*> SplineProperties;
	
	FDelegateHandle DeselectedInEditorDelegateHandle;

	void UpdateSharedAttributeNames() const
	{
		if (const USplineComponent* SplineComp = GetEditedSplineComponent())
		{
			SharedAttributeNames.Empty();
			SharedAttributeNames.Add(MakeShared<FName>(NAME_None));
			Algo::Transform(SplineComp->GetFloatPropertyChannels(), SharedAttributeNames, [](const FName& Name)
			{
				return MakeShared<FName>(Name);
			});
		}
	}
	mutable TArray<TSharedPtr<FName>> SharedAttributeNames;
	
private:

	FDelegateHandle OnObjectTransactedHandle;

	mutable TMap<TWeakObjectPtr<const USplineComponent>, FPDICache> PDICache;
	void DirtyPDICache(const USplineComponent* SplineComp);
	void UpdatePDICache(const USplineComponent* SplineComp);
	
	TSharedPtr<SSplineGeneratorPanel> SplineGeneratorPanel;
	static UE_API TWeakPtr<SWindow> WeakExistingWindow;
};

/** This class could be changed or removed without deprecation, use at your own risk. */
class UE_INTERNAL ISplineComponentVisualizerSuppressor;

PRAGMA_DISABLE_INTERNAL_WARNINGS
/** Allows implements to suppress the visualizer (disabling rendering, input capture, etc.) for a particular spline component. */
class ISplineComponentVisualizerSuppressor : public IModularFeature
{
public:

	virtual ~ISplineComponentVisualizerSuppressor() = default;
	
	static FName GetModularFeatureName() { return FName(TEXT("SplineComponentVisualizerSuppressor")); }
	
	virtual bool ShouldSuppress(const USplineComponent* SplineComponent) const = 0;
};
PRAGMA_ENABLE_INTERNAL_WARNINGS

#undef UE_API
