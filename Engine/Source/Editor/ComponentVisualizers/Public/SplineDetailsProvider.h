// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Features/IModularFeature.h"
#include "UObject/NameTypes.h"

class USplineComponent;

class UE_INTERNAL ISplineDetailsProvider;

/**
 * Interface for spline details panels to interact with spline editing tools.
 * 
 * This interface was extracted from FSplineComponentVisualizer to integrate a
 * replacement of that visualizer into the spline component details customization.
 */
class ISplineDetailsProvider : public IModularFeature
{
public:
	virtual ~ISplineDetailsProvider() = default;

	/** Get the modular feature name for registration/lookup */
	static FName GetModularFeatureName() 
	{ 
		return FName(TEXT("SplineDetailsInterface")); 
	}

	/** Check if this implementation should be used for the given spline component */
	virtual bool ShouldUseForSpline(const USplineComponent* SplineComponent) const = 0;

	/** Get the currently selected spline point indices */
	virtual const TSet<int32>& GetSelectedKeys() const = 0;

	/** Get the spline component currently being edited by this system */
	virtual USplineComponent* GetEditedSplineComponent() const = 0;

	/** Select the first or last spline point */
	virtual bool HandleSelectFirstLastSplinePoint(USplineComponent* SplineComp, bool bFirst) = 0;

	/** Select the previous or next spline point */
	virtual void HandleSelectPrevNextSplinePoint(bool bNext, bool bAddToSelection) = 0;

	/** Select all spline points */
	virtual bool HandleSelectAllSplinePoints(USplineComponent* SplineComp) = 0;

	/** Set the cached rotation for the spline component */
	virtual void SetCachedRotation(const FQuat& NewRotation) {}

	/** Activate the visualization for the edited spline component */
	virtual void ActivateVisualization() {}
};
