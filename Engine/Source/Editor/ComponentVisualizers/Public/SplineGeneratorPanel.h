// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ClassViewerFilter.h"
#include "ClassViewerModule.h"
#include "Components/SplineComponent.h"
#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "Layout/WidgetPath.h"
#include "Misc/EnumClassFlags.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "SplineComponentVisualizer.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWindow.h"

#include "SplineGeneratorPanel.generated.h"

class FSplineComponentVisualizer;
class IDetailsView;
class SWindow;
struct FFocusEvent;
struct FGeometry;


UENUM()
enum class EShapeAddMode : uint8
{
	AppendAfter = 0x01,
	AppendBefore = 0x02,
	InsertAfter = 0x04,
	InsertBefore = 0x08
};

ENUM_CLASS_FLAGS(EShapeAddMode);

UCLASS(config = EditorSettings)
class USplineGeneratorBase : public UObject
{
	GENERATED_BODY()
public:

	USplineGeneratorBase()
		: ShapeAddMode(EShapeAddMode::AppendAfter)
	{}

	void Init(TWeakPtr<FSplineComponentVisualizer> InSplineComponentVisualizer);
	void Reset();
	void PreviewCurve();

	virtual void BuildCurve() {}
	virtual int32 GetNumPoints() const { return 0; }

	UPROPERTY(Transient, EditAnywhere, Category = GenerateOptions, Meta = (ToolTip = "How to add the shape to the selection"))
	EShapeAddMode ShapeAddMode;

	int32 StartKey;

protected:
	/** Helper method to return the index for adding a spline point */
	int32 GetAddIndex(int32 Index) const;
	/** Helper method to return the index for an added spline point after all points have been added */
	int32 GetItrIndex(int32 Index) const;

	FSplineCurves CachedSplineCurves;

	TWeakObjectPtr<USplineComponent> SelectedSplineComponent;
	TWeakPtr<FSplineComponentVisualizer> WeakSplineVis;
};

UCLASS(config = EditorSettings, DisplayName = "Circle")
class UCircleSplineGenerator : public USplineGeneratorBase
{
	GENERATED_BODY()

public:
	UCircleSplineGenerator()
		: NumberOfPoints(3)
		, Radius(100.f)
		, bReverseDir(false)
		, bKeepFirstKeyTangent(true)
		, bBranchRight(false)
	{}

	virtual void BuildCurve() override;

	virtual int32 GetNumPoints() const override { return NumberOfPoints; }

	UPROPERTY(Transient, EditAnywhere, Category = ShapeParameters, Meta = (ToolTip = "Number of points making up shape", ClampMin = "2"))
	int32 NumberOfPoints;

	UPROPERTY(Transient, EditAnywhere, Category = ShapeParameters, Meta=(ToolTip="Radius of circle", ClampMin = "0"))
	float Radius;

	UPROPERTY(Transient, EditAnywhere, Category = OtherParameters, Meta=(ToolTip="If enabled, will reverse the direction of the circle"))
	bool bReverseDir;

	UPROPERTY(Transient, EditAnywhere, Category = OtherParameters, Meta=(ToolTip="If enabled, will start the shape tangent to the current path"))
	bool bKeepFirstKeyTangent;

	UPROPERTY(Transient, EditAnywhere, Category = OtherParameters, Meta=(ToolTip = "If enabled, will switch the circle's center to the right of the curve", EditCondition="bKeepFirstKeyTangent"))
	bool bBranchRight;
};

UCLASS(config = EditorSettings, DisplayName = "Arc")
class UArcSplineGenerator : public USplineGeneratorBase
{
	GENERATED_BODY()

public:
	UArcSplineGenerator()
		: NumberOfPoints(4)
		, Radius(100.f)
		, Degrees(90.f)
		, bReverseDir(false)
		, bKeepFirstKeyTangent(true)
		, bBranchRight(false)
	{}

	virtual void BuildCurve() override;

	virtual int32 GetNumPoints() const override { return NumberOfPoints; }

	UPROPERTY(Transient, EditAnywhere, Category = ShapeParameters, Meta = (ToolTip = "Number of points making up shape", ClampMin = "2"))
	int32 NumberOfPoints;

	UPROPERTY(Transient, EditAnywhere, Category = ShapeParameters, Meta = (ToolTip = "Radius of arc", ClampMin = "0"))
	float Radius;

	UPROPERTY(Transient, EditAnywhere, Category = ShapeParameters, Meta = (ToolTip = "Degree of arc", ClampMin = "0"))
	float Degrees;

	UPROPERTY(Transient, EditAnywhere, Category = OtherParameters, Meta = (ToolTip = "If enabled, will reverse the direction of the arc"))
	bool bReverseDir;

	UPROPERTY(Transient, EditAnywhere, Category = OtherParameters, Meta = (ToolTip = "If enabled, will start the shape tangent to the current path"))
	bool bKeepFirstKeyTangent;

	UPROPERTY(Transient, EditAnywhere, Category = OtherParameters, Meta = (ToolTip = "If enabled, will switch the arc's center to the right of the curve", EditCondition = "bKeepFirstKeyTangent"))
	bool bBranchRight;
};

UCLASS(config = EditorSettings, DisplayName = "Square")
class USquareSplineGenerator : public USplineGeneratorBase
{
	GENERATED_BODY()

public:
	USquareSplineGenerator()
		: Length(100.f)
		, bBranchRight(false)
	{}

	virtual void BuildCurve() override;

	virtual int32 GetNumPoints() const override { return 4; }

	UPROPERTY(Transient, EditAnywhere, Category = ShapeParameters, Meta = (ClampMin = "0"))
	float Length;

	UPROPERTY(Transient, EditAnywhere, Category = OtherParameters, Meta = (ToolTip = "If enabled, will switch the shape's center to the right of the curve"))
	bool bBranchRight;
};

UCLASS(config = EditorSettings, DisplayName = "Ellipse")
class UEllipseSplineGenerator : public USplineGeneratorBase
{
	GENERATED_BODY()

public:
	UEllipseSplineGenerator()
		: NumberOfPoints(10)
		, Length(200.f)
		, Width(200.f)
		, bReverseDir(false)
		, bKeepFirstKeyTangent(true)
		, bBranchRight(false)
	{}

	virtual void BuildCurve() override;

	UPROPERTY(Transient, EditAnywhere, Category = ShapeParameters, Meta = (ClampMin = "2"))
	int32 NumberOfPoints;

	UPROPERTY(Transient, EditAnywhere, Category = ShapeParameters, Meta = (ClampMin = "0"))
	float Length;

	UPROPERTY(Transient, EditAnywhere, Category = ShapeParameters, Meta = (ClampMin = "0"))
	float Width;

	UPROPERTY(Transient, EditAnywhere, Category = OtherParameters, Meta = (ToolTip = "If enabled, will reverse the direction of the arc"))
	bool bReverseDir;

	UPROPERTY(Transient, EditAnywhere, Category = OtherParameters, Meta = (ToolTip = "If enabled, will start the shape tangent to the current path"))
	bool bKeepFirstKeyTangent;

	UPROPERTY(Transient, EditAnywhere, Category = OtherParameters, Meta = (ToolTip = "If enabled, will switch the shape's center to the right of the curve", EditCondition = "bKeepFirstKeyTangent"))
	bool bBranchRight;
};

UCLASS(config = EditorSettings, DisplayName = "Rectangle")
class URectangleSplineGenerator : public USplineGeneratorBase
{
	GENERATED_BODY()

public:
	URectangleSplineGenerator()
		: Length(100.f)
		, Width(100.f)
		, bBranchRight(false)
	{}

	virtual void BuildCurve() override;

	virtual int32 GetNumPoints() const override { return 4; }

	UPROPERTY(Transient, EditAnywhere, Category = ShapeParameters, Meta = (ClampMin = "0"))
	float Length;

	UPROPERTY(Transient, EditAnywhere, Category = ShapeParameters, Meta = (ClampMin = "0"))
	float Width;

	UPROPERTY(Transient, EditAnywhere, Category = OtherParameters, Meta = (ToolTip = "If enabled, will switch the shape's center to the right of the curve"))
	bool bBranchRight;
};

UCLASS(config = EditorSettings, DisplayName = "Line")
class ULineSplineGenerator : public USplineGeneratorBase
{
	GENERATED_BODY()

public:
	ULineSplineGenerator()
		: NumberOfPoints(5)
		, Length(100.f)
		, bEnableUpToNextPoint(false)
		, bUpToNextPoint(false)
	{}

	virtual void BuildCurve() override;

	virtual int32 GetNumPoints() const override { return NumberOfPoints; };

	UPROPERTY(Transient, EditAnywhere, Category = ShapeParameters, Meta = (ClampMin = "1"))
	int32 NumberOfPoints;

	UPROPERTY(Transient, EditAnywhere, Category = ShapeParameters, Meta = (ClampMin = "0", EditCondition = "!bUpToNextPoint"))
	double Length;

	UPROPERTY()
	bool bEnableUpToNextPoint;

	UPROPERTY(Transient, EditAnywhere, Category = ShapeParameters, Meta = (ToolTip = "If enabled, will add points up until the next existing point", EditCondition = "bEnableUpToNextPoint"))
	bool bUpToNextPoint;

};

class SSplineGeneratorPanel : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SSplineGeneratorPanel)
	{}

	SLATE_END_ARGS()

	/** SWidget Interface */
	void Construct(const FArguments& InArgs, TWeakPtr<FSplineComponentVisualizer> InWeakSplineComponentVisualizer);
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	virtual void OnFocusChanging(const FWeakWidgetPath& PreviousFocusPath, const FWidgetPath& NewWidgetPath, const FFocusEvent& InFocusEvent) override;

	/** SWindow Interface */
	void OnWindowClosed(const TSharedRef<SWindow>&);

	void OnSelectionUpdated();

	~SSplineGeneratorPanel();

private:

	TSharedPtr<IDetailsView> DetailView;

	TWeakPtr<FSplineComponentVisualizer> WeakSplineComponentVisualizer;
	USplineGeneratorBase* SplineGen;

	TArray<UObject*> ShapeGenRegistry;

	/** The currently open transaction (if any) */
	TUniquePtr<class FScopedTransaction> ActiveTransaction;
};