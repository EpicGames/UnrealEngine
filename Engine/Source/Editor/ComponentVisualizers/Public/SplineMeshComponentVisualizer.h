// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ComponentVisualizer.h"
#include "CoreMinimal.h"
#include "Engine/EngineBaseTypes.h"
#include "HitProxies.h"
#include "InputCoreTypes.h"
#include "Math/InterpCurve.h"
#include "Math/Matrix.h"
#include "Math/Rotator.h"
#include "Math/UnrealMathSSE.h"
#include "Templates/SharedPointer.h"

#define UE_API COMPONENTVISUALIZERS_API

class AActor;
class FEditorViewportClient;
class FPrimitiveDrawInterface;
class FSceneView;
class FViewport;
class SWidget;
class UActorComponent;
class USplineMeshComponent;
struct FViewportClick;

/** Base class for clickable spline mesh component editing proxies */
struct HSplineMeshVisProxy : public HComponentVisProxy
{
	DECLARE_HIT_PROXY( UE_API );

	HSplineMeshVisProxy(const UActorComponent* InComponent)
	: HComponentVisProxy(InComponent, HPP_Wireframe)
	{}
};

/** Proxy for a spline mesh component key */
struct HSplineMeshKeyProxy : public HSplineMeshVisProxy
{
	DECLARE_HIT_PROXY( UE_API );

	HSplineMeshKeyProxy(const UActorComponent* InComponent, int32 InKeyIndex) 
		: HSplineMeshVisProxy(InComponent)
		, KeyIndex(InKeyIndex)
	{}

	int32 KeyIndex;
};

/** Proxy for a tangent handle */
struct HSplineMeshTangentHandleProxy : public HSplineMeshVisProxy
{
	DECLARE_HIT_PROXY( UE_API );

	HSplineMeshTangentHandleProxy(const UActorComponent* InComponent, int32 InKeyIndex, bool bInArriveTangent)
		: HSplineMeshVisProxy(InComponent)
		, KeyIndex(InKeyIndex)
		, bArriveTangent(bInArriveTangent)
	{}

	int32 KeyIndex;
	bool bArriveTangent;
};

/** SplineMeshComponent visualizer/edit functionality */
class FSplineMeshComponentVisualizer : public FComponentVisualizer
{
public:
	UE_API FSplineMeshComponentVisualizer();
	UE_API virtual ~FSplineMeshComponentVisualizer();

	//~ Begin FComponentVisualizer Interface
	UE_API virtual void OnRegister() override;
	UE_API virtual void DrawVisualization(const UActorComponent* Component, const FSceneView* View, FPrimitiveDrawInterface* PDI) override;
	UE_API virtual bool VisProxyHandleClick(FEditorViewportClient* InViewportClient, HComponentVisProxy* VisProxy, const FViewportClick& Click) override;
	UE_API virtual void EndEditing() override;
	UE_API virtual bool GetWidgetLocation(const FEditorViewportClient* ViewportClient, FVector& OutLocation) const override;
	UE_API virtual bool GetCustomInputCoordinateSystem(const FEditorViewportClient* ViewportClient, FMatrix& OutMatrix) const override;
	UE_API virtual bool HandleInputDelta(FEditorViewportClient* ViewportClient, FViewport* Viewport, FVector& DeltaTranslate, FRotator& DeltaRotate, FVector& DeltaScale) override;
	UE_API virtual bool HandleInputKey(FEditorViewportClient* ViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event) override;
	UE_API virtual TSharedPtr<SWidget> GenerateContextMenu() const override;
	//~ End FComponentVisualizer Interface

	/** Get the spline component we are currently editing */
	UE_API USplineMeshComponent* GetEditedSplineMeshComponent() const;

protected:

	/** Get a spline object for the specified spline mesh component */
	UE_API FInterpCurveVector GetSpline(const USplineMeshComponent* SplineMeshComp) const;

	/** Syncs changes made by the visualizer in the actual component */
	UE_API void NotifyComponentModified();

	/** Property path from the parent actor to the component */
	FComponentPropertyPath SplineMeshPropertyPath;

	/** Index of the key we selected */
	int32 SelectedKey;

	/** Index of tangent handle we have selected */
	int32 SelectedTangentHandle;

	struct ESelectedTangentHandle
	{
		enum Type
		{
			None,
			Leave,
			Arrive
		};
	};

	/** The type of the selected tangent handle */
	ESelectedTangentHandle::Type SelectedTangentHandleType;
};

#undef UE_API
